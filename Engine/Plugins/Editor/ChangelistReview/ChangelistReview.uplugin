{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Changelist Reviews",
	"Description": "Review source control changelists",
	"Category": "Source Control",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"DocsURL": "",
	"EnabledByDefault": true,
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Plugins": [
	],
	"Modules": [
		{
			"Name": "ChangelistReview",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}
