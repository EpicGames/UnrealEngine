// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ChangelistReview : ModuleRules
{
	public ChangelistReview(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Engine",
				"CoreUObject",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"SlateCore",
				"Slate",
				"EditorFramework",
				"UnrealEd",
				"Kismet",
				"EditorStyle",
				"ToolMenus",
				"SourceControl",
				"InputCore",
				"HTTP",
				"Json"
			}
		);

		AddEngineThirdPartyPrivateStaticDependencies(Target, "Perforce");

		if (Target.Platform == UnrealTargetPlatform.Win64 || Target.Platform == UnrealTargetPlatform.Mac)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenSSL");
			AddEngineThirdPartyPrivateStaticDependencies(Target, "zlib");
		}
	}
}