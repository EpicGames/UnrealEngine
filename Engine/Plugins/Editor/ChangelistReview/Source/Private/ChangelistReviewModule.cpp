// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChangelistReviewModule.h"
#include "ISourceControlModule.h"
#include "ToolMenu.h"
#include "Widgets/Docking/SDockTab.h"
#include "SSourceControlReview.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "ChangelistReviewModule"

static const FName SourceControlReviewTabName = FName(TEXT("SourceControlChangelistReview"));

namespace UE::DiffControl
{
	extern KISMET_API const TArray<FReviewComment>*(*GGetReviewCommentsForFile)(const FString&);
	extern KISMET_API void(*GPostReviewComment)(FReviewComment&);
	extern KISMET_API void(*GEditReviewComment)(FReviewComment&);
	extern KISMET_API FString(*GGetReviewerUsername)(void);
	extern KISMET_API bool (*GIsFileInReview)(const FString& File);
}

static const TArray<FReviewComment>* GetReviewCommentsForFile(const FString& FilePath)
{
	const TWeakPtr<SSourceControlReview> ReviewWeak = FChangelistReviewModule::Get().GetActiveReview();
	if (const TSharedPtr<SSourceControlReview> Review = ReviewWeak.Pin())
	{
		return Review->GetReviewCommentsForFile(FilePath);
	}
	return nullptr;
}

static void PostReviewComment(FReviewComment& Comment)
{
	const TWeakPtr<SSourceControlReview> ReviewWeak = FChangelistReviewModule::Get().GetActiveReview();
	if (const TSharedPtr<SSourceControlReview> Review = ReviewWeak.Pin())
	{
		Review->PostComment(Comment);
	}
}

static void EditReviewComment(FReviewComment& Comment)
{
	const TWeakPtr<SSourceControlReview> ReviewWeak = FChangelistReviewModule::Get().GetActiveReview();
	if (const TSharedPtr<SSourceControlReview> Review = ReviewWeak.Pin())
	{
		Review->EditComment(Comment);
	}
}

static FString GetReviewerUsername()
{
	const TWeakPtr<SSourceControlReview> ReviewWeak = FChangelistReviewModule::Get().GetActiveReview();
	if (const TSharedPtr<SSourceControlReview> Review = ReviewWeak.Pin())
	{
		return Review->GetReviewerUsername();
	}
	return {};
}

static bool IsFileInReview(const FString& File)
{
	const TWeakPtr<SSourceControlReview> ReviewWeak = FChangelistReviewModule::Get().GetActiveReview();
	if (const TSharedPtr<SSourceControlReview> Review = ReviewWeak.Pin())
	{
		return Review->IsFileInReview(File);
	}
	return false;
}

void FChangelistReviewModule::StartupModule()
{
	UToolMenu* SourceControlMenu = UToolMenus::Get()->ExtendMenu("StatusBar.ToolBar.SourceControl");
	FToolMenuSection& Section = SourceControlMenu->AddSection("SourceControlActions", LOCTEXT("SourceControlMenuHeadingActions", "Actions"));
	
	const FSlateIcon SourceControlIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.ChangelistsTab");
	Section.AddMenuEntry(
		FName(TEXT("Review Changelists")),
		LOCTEXT("ReviewChangelist_Label", "Review Changelists"),
		LOCTEXT("ReviewChangelist_Tooltip", "Opens a dialog to review shelved and submitted changelists"),
		SourceControlIcon,
		FUIAction(
			FExecuteAction::CreateRaw(this, &FChangelistReviewModule::ShowReviewTab),
			FCanExecuteAction(),
			FGetActionCheckState(),
			FIsActionButtonVisible::CreateRaw(this, &FChangelistReviewModule::CanShowReviewTab)
		)
	);
	
	// Register the changelist tab spawner
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(SourceControlReviewTabName, FOnSpawnTab::CreateRaw(this, &FChangelistReviewModule::CreateReviewTab))
		.SetDisplayName(LOCTEXT("ChangelistsTabTitle", "Review Changelist"))
		.SetTooltipText(LOCTEXT("ChangelistsTabTooltip", "Opens a dialog to review shelved and submitted changelists."))
		.SetIcon(SourceControlIcon)
		.SetMenuType(ETabSpawnerMenuType::Hidden);

	UE::DiffControl::GGetReviewCommentsForFile = GetReviewCommentsForFile;
	UE::DiffControl::GPostReviewComment = PostReviewComment;
	UE::DiffControl::GEditReviewComment = EditReviewComment;
	UE::DiffControl::GGetReviewerUsername = GetReviewerUsername;
	UE::DiffControl::GIsFileInReview = IsFileInReview;
}

void FChangelistReviewModule::ShutdownModule()
{
	UE::DiffControl::GGetReviewCommentsForFile = nullptr;
	UE::DiffControl::GPostReviewComment = nullptr;
	UE::DiffControl::GEditReviewComment = nullptr;
	UE::DiffControl::GGetReviewerUsername = nullptr;
	UE::DiffControl::GIsFileInReview = nullptr;
}

TSharedRef<SDockTab> FChangelistReviewModule::CreateReviewTab(const FSpawnTabArgs& Args)
{
	return SAssignNew(ReviewTab, SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			CreateReviewUI().ToSharedRef()
		];
}

TSharedPtr<SWidget> FChangelistReviewModule::CreateReviewUI()
{
	TSharedPtr<SSourceControlReview> ReturnWidget;
	if (IsInGameThread())
	{
		ReturnWidget = SNew(SSourceControlReview);
		ReviewWidget = ReturnWidget;
	}
	return ReturnWidget;
}

void FChangelistReviewModule::ShowReviewTab()
{
	FGlobalTabmanager::Get()->TryInvokeTab(FTabId(SourceControlReviewTabName));
}

bool FChangelistReviewModule::CanShowReviewTab() const
{
	const ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
	if (SourceControlModule.IsEnabled())
	{
		const ISourceControlProvider& Provider = SourceControlModule.GetProvider();
	
		// we currently only support perforce for the review tool
		return Provider.IsAvailable() && Provider.GetName().ToString() == TEXT("Perforce");
	}
	return false;
}

TWeakPtr<SSourceControlReview> FChangelistReviewModule::GetActiveReview()
{
	return ReviewWidget;
}

bool FChangelistReviewModule::OpenChangelistReview(const FString& Changelist)
{
	TSharedPtr<SDockTab> DockTab = FGlobalTabmanager::Get()->TryInvokeTab(FTabId(SourceControlReviewTabName));
	if (!DockTab)
	{
		return false;
	}

	TSharedPtr<SWidget> ReviewWidgetFromTab = DockTab->GetContent();
	if (!ReviewWidgetFromTab)
	{
		return false;
	}

	TSharedPtr<SSourceControlReview> ReviewTool = StaticCastSharedPtr<SSourceControlReview>(ReviewWidgetFromTab);
	if (!ReviewTool)
	{
		return false;
	}

	if (ReviewTool->OpenChangelist(Changelist))
	{
		DockTab->DrawAttention();
		return true;
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FChangelistReviewModule, ChangelistReview)
