// Copyright Epic Games, Inc. All Rights Reserved.
#include "FSwarmCommentsAPI.h"

#include "HttpModule.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "SourceControlOperations.h"
#include "Interfaces/IHttpResponse.h"
#include "Misc/Base64.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/JsonSerializer.h"

THIRD_PARTY_INCLUDES_START
// used to retrieve the swarm url from p4
#include <p4/clientapi.h>
THIRD_PARTY_INCLUDES_END

namespace UE::SwarmCommentsAPI::Keywords
{
	// http
	static const TCHAR* HttpPost = TEXT("POST");
	static const TCHAR* HttpGet = TEXT("GET");
	static const TCHAR* HttpPatch = TEXT("PATCH");

	// swarm
	static const TCHAR* SilenceNotification = TEXT("silenceNotification");
	static const TCHAR* True = TEXT("true");
	static const TCHAR* False = TEXT("false");

	// metadata
	static const TCHAR* MetadataTag = TEXT("[UE_Metadata]");
	static const TCHAR* MetadataBody = TEXT("Body");
	static const TCHAR* MetadataFile = TEXT("File");
	static const TCHAR* MetadataReplyTo = TEXT("ReplyTo");
	static const TCHAR* MetadataCategory = TEXT("Category");

	// userdata
	static const TCHAR* UserdataTag = TEXT("[UE_Userdata]");
	static const TCHAR* UserdataLikedComments = TEXT("LikedComments");
	static const TCHAR* UserdataReadComments = TEXT("ReadComments");
}
using namespace UE::SwarmCommentsAPI;

static TMap<FString, FString> ParseReviewCommentMetadata(const FString& Comment, const FString& MetadataKey = Keywords::MetadataTag)
{
	TMap<FString, FString> Result;
	FString Body, Metadata;
	if (!Comment.Split(MetadataKey, &Body, &Metadata))
	{
		return {{Keywords::MetadataBody, Comment.TrimEnd()}};
	}
	
	Result.Add(Keywords::MetadataBody, Body.TrimEnd());
	
	TArray<FString> MetaDataLines;
	Metadata.ParseIntoArrayLines(MetaDataLines);
	for (const FString& Line : MetaDataLines)
	{
		const int32 KeyEnd = Line.Find(TEXT("="), ESearchCase::CaseSensitive, ESearchDir::FromStart);
		if (KeyEnd == INDEX_NONE)
		{
			continue;
		}
		FString Key = Line.Left(KeyEnd);
		
		const int32 ValBegin = KeyEnd + 1; // skip the '=' char
		const int32 ValLen = Line.Len() - ValBegin; // extend to end of line
		const FString Val = Line.Mid(ValBegin, ValLen);
		Result.Add(Key, Val);
	}
	return Result;
}

static TMap<FString, FString> ParseReviewCommentMetadata(const FReviewComment& Comment, const FString& MetadataKey = Keywords::MetadataTag )
{
	return ParseReviewCommentMetadata(Comment.Body.Get({}), MetadataKey);
}

static void SetReviewCommentMetadata(FReviewComment& Comment, TMap<FString, FString> Metadata, const FString& MetadataKey = Keywords::MetadataTag)
{
	FString CommentText;
	if (FString* Body = Metadata.Find(Keywords::MetadataBody))
	{
		CommentText = *Body;
		Metadata.Remove(Keywords::MetadataBody);
	}
	
	if (Metadata.IsEmpty())
	{
		Comment.Body = CommentText;
		return;
	}
	
	CommentText += TEXT("\n\n")+MetadataKey+TEXT("\n");
	for (auto&[Key,Value] : Metadata)
	{
		if (Key != Keywords::MetadataBody)
		{
			CommentText += FString::Format(TEXT("{0}={1}\n"), {Key, Value});
		}
	}
	Comment.Body = CommentText;
}

// turn a comma separated string into a set of strings
static TSet<FString> DeserializeSetString(const FString& SetString)
{
	TSet<FString> Result;
	TArray<FString> LikesArray;
	SetString.ParseIntoArray(LikesArray, TEXT(","));
	for (const FString& Like : LikesArray)
	{
		Result.Add(Like);
	}
	return Result;
}

// turn a set of strings into a comma separated string
static FString SerializeSetString(const TSet<FString>& Set)
{
	FString Result;
	int32 Index = 0;
	for (const FString& Item : Set)
	{
		Result += Item;
		if (Index != Set.Num() - 1)
		{
			Result += TEXT(",");
		}
		++Index;
	}
	return Result;
}

FSwarmCommentsAPI::FSwarmCommentsAPI(const FString& AuthTicket, const FString& SwarmURL)
	: AuthTicket(AuthTicket)
	, SwarmURL(SwarmURL)
{
}

FSwarmCommentsAPI::FSwarmCommentsAPI(const FAuthTicket& AuthTicket, const FString& SwarmURL)
	: AuthTicket(AuthTicket)
	, SwarmURL(SwarmURL)
{}

TSharedPtr<FSwarmCommentsAPI> FSwarmCommentsAPI::TryConnect()
{
	FAuthTicket AuthTicket = RetrieveAuthorizationTicket();
	if (!AuthTicket.IsValid())
	{
		return nullptr;
	}
	FString SwarmURL = RetrieveSwarmURL(AuthTicket.Username);
	if (SwarmURL.IsEmpty())
	{
		return nullptr;
	}
	return MakeShared<FSwarmCommentsAPI>(AuthTicket, SwarmURL);
}

FString FSwarmCommentsAPI::GetUsername() const
{
	return AuthTicket.Username;
}

void FSwarmCommentsAPI::GetComments(const FReviewTopic& Topic, const OnGetCommentsComplete& OnComplete) const
{
    const TSharedRef<IHttpRequest> HttpRequest = FHttpModule::Get().CreateRequest();
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json; charset=utf-8"));
	HttpRequest->SetHeader(TEXT("Authorization"), AuthTicket);
	
    HttpRequest->SetURL(FString::Format(TEXT("{0}?topic={1}&max={2}"),
    	{
    		CommentsURL(),
    		Topic.ToString(), // filter to comments in a specific review
    		5000 // limit to 5k comments. This should obviously be more than enough and loads at a reasonable rate
    	}));
    HttpRequest->SetVerb(Keywords::HttpGet);

	TWeakPtr<const FSwarmCommentsAPI> WeakSelf = AsWeak();

	auto BuildCommentArray = [WeakSelf, Topic](const TArray<TSharedPtr<FJsonValue>>& CommentsJson)->TArray<FReviewComment>
	{
		bool bUserDataCacheFound = false;
		TArray<FReviewComment> Comments;
		
		TMap<uint32, TSet<FString>> CommentLikes;
		TMap<uint32, TSet<FString>> CommentReadBy;

		const TSharedPtr<const FSwarmCommentsAPI> Self = WeakSelf.Pin();
		if (!Self)
		{
			return {};
		}
		
		for (const TSharedPtr<FJsonValue>& CommentJson : CommentsJson)
		{
			FReviewComment Comment = FReviewComment::FromJson(CommentJson->AsObject());
			if (FSwarmUserdataComment::IsUserdataComment(Comment))
			{
				FSwarmUserdataComment AsUserData = Comment;
				if (Comment.User == Self->GetUsername())
				{
					bUserDataCacheFound = true;
					Self->UserdataCache.Add(Topic, AsUserData);
				}
				
				for (const FString& LikedCommentID : AsUserData.GetLikedComments())
				{
					CommentLikes.FindOrAdd(FCString::Atoi(*LikedCommentID)).Add(*Comment.User);
				}
				for (const FString& ReadCommentID : AsUserData.GetReadComments())
				{
					CommentReadBy.FindOrAdd(FCString::Atoi(*ReadCommentID)).Add(*Comment.User);
				}
				
				continue; // don't include userdata comments in the comment list
			}
			
			TakeMetadataFromBody(Comment);
			Comments.Add(Comment);
		}

		if (!bUserDataCacheFound)
		{
			// every user should have a comment that stores their userdata
			Self->PostNewUserdataComment(Topic);
		}
		
		// parse Userdata for Likes and ReadBy state and update the associated comments
		for (FReviewComment& Comment : Comments)
		{
			if (const TSet<FString>* Likes = CommentLikes.Find(*Comment.CommentID))
			{
				if (!Comment.Likes.IsSet() && !Likes->IsEmpty())
				{
					Comment.Likes = TSet<FString>{};
				}
				for (const FString& Like : *Likes)
				{
					Comment.Likes->Add(Like);
				}
			}
			if (const TSet<FString>* Reads = CommentReadBy.Find(*Comment.CommentID))
			{
				if (!Comment.ReadBy.IsSet() && !Reads->IsEmpty())
				{
					Comment.ReadBy = TSet<FString>{};
				}
				for (const FString& ReadBy : *Reads)
				{
					Comment.ReadBy->Add(ReadBy);
				}
			}
		}
		
		return Comments;
	};
	
	HttpRequest->OnProcessRequestComplete().BindLambda([BuildCommentArray, OnComplete](FHttpRequestPtr Request, FHttpResponsePtr Response, bool bConnectedSuccessfully)
	{
		if (!bConnectedSuccessfully)
		{
			OnComplete.ExecuteIfBound({}, TEXT("Connection Failed"));
			return;
		}
		const TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(Response->GetContentAsString());
		TSharedPtr<FJsonObject> JsonObject;
		if (FJsonSerializer::Deserialize(JsonReader, JsonObject))
		{
			if (JsonObject->HasField(TEXT("error")))
			{
				OnComplete.ExecuteIfBound({}, JsonObject->GetStringField(TEXT("error")));
				return;
			}
			if (JsonObject->HasField(TEXT("comments")))
			{
				OnComplete.ExecuteIfBound(BuildCommentArray(JsonObject->GetArrayField(TEXT("comments"))), {});
				return;
			}
		}
		OnComplete.ExecuteIfBound({}, TEXT("Malformed Response"));
	});
		
	
	HttpRequest->ProcessRequest();
}

void FSwarmCommentsAPI::PostComment(FReviewComment& Comment, const OnPostCommentComplete& OnComplete, bool bSilenceNotification) const
{
	const TSharedRef<IHttpRequest> HttpRequest = FHttpModule::Get().CreateRequest();
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json; charset=utf-8"));
	HttpRequest->SetHeader(TEXT("Authorization"), AuthTicket);
	HttpRequest->SetURL(CommentsURL());
	HttpRequest->SetVerb(Keywords::HttpPost);

	Comment.User = AuthTicket.Username;
	FReviewComment CleanedComment = Comment;
	PutMetadataInBody(CleanedComment);
	const TSharedPtr<FJsonObject> JsonObject = CleanedComment.ToJsonObject();
	if (bSilenceNotification)
	{
		JsonObject->SetStringField(Keywords::SilenceNotification, Keywords::True);
	}

	FString JsonString;
	const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&JsonString);
	FJsonSerializer::Serialize(JsonObject.ToSharedRef(), Writer);
	
	HttpRequest->SetContentAsString(JsonString);
	
	HttpRequest->OnProcessRequestComplete().BindLambda([OnComplete, WeakSelf = AsWeak()](FHttpRequestPtr Request, FHttpResponsePtr Response, bool bConnectedSuccessfully)
	{
		if (!bConnectedSuccessfully)
		{
			OnComplete.ExecuteIfBound({}, TEXT("Connection Failed"));
			return;
		}
		const TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(Response->GetContentAsString());
		TSharedPtr<FJsonObject> JsonObject;
		if (FJsonSerializer::Deserialize(JsonReader, JsonObject))
		{
			if (JsonObject->HasField(TEXT("error")))
			{
				OnComplete.ExecuteIfBound({}, JsonObject->GetStringField(TEXT("error")));
				return;
			}
			if (JsonObject->HasField(TEXT("comment")))
			{
				FReviewComment Comment = FReviewComment::FromJson(JsonObject->GetObjectField(TEXT("comment")));
				if (TSharedPtr<const FSwarmCommentsAPI> Self = WeakSelf.Pin())
				{
					// This comment may effect the userdata. update it if necessary
					WeakSelf.Pin()->UpdateUserdata(Comment);
				}
				TakeMetadataFromBody(Comment);
				OnComplete.ExecuteIfBound(Comment, {});
				return;
			}
		}
		OnComplete.ExecuteIfBound({}, TEXT("Malformed Response"));
	});
	
	HttpRequest->ProcessRequest();
}

void FSwarmCommentsAPI::EditComment(const FReviewComment& Comment, const OnEditCommentComplete& OnComplete, bool bSilenceNotification) const
{
	// edits to this comment may effect the userdata. update it if necessary
	UpdateUserdata(Comment);
	
	// you can't edit other people's comments. Assume that only likes or ReadBy were edited
	if (Comment.User != GetUsername())
	{
		return;
	}

	// edits without bodies should only update Userdata
	if (!Comment.Body.IsSet())
	{
		return;
	}
	
	const TSharedRef<IHttpRequest> HttpRequest = FHttpModule::Get().CreateRequest();
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json; charset=utf-8"));
	HttpRequest->SetHeader(TEXT("Authorization"), AuthTicket);
	const FString CommentID = FString::FromInt(*Comment.CommentID);
	HttpRequest->SetURL(CommentsURL() / CommentID);
	HttpRequest->SetVerb(Keywords::HttpPatch);
	
	FReviewComment CleanedComment = Comment;
	PutMetadataInBody(CleanedComment);
	
	const TSharedPtr<FJsonObject> JsonObject = CleanedComment.ToJsonObject();
	if (bSilenceNotification)
	{
		JsonObject->SetStringField(Keywords::SilenceNotification, Keywords::True);
	}

	FString JsonString;
	const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&JsonString);
	FJsonSerializer::Serialize(JsonObject.ToSharedRef(), Writer);
	
	HttpRequest->SetContentAsString(JsonString);
	HttpRequest->OnProcessRequestComplete().BindLambda([OnComplete](FHttpRequestPtr Request, FHttpResponsePtr Response, bool bConnectedSuccessfully)
	{
		if (!bConnectedSuccessfully)
		{
			OnComplete.ExecuteIfBound({}, TEXT("Connection Failed"));
			return;
		}
		const TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(Response->GetContentAsString());
		TSharedPtr<FJsonObject> JsonObject;
		if (FJsonSerializer::Deserialize(JsonReader, JsonObject))
		{
			if (JsonObject->HasField(TEXT("error")))
			{
				OnComplete.ExecuteIfBound({}, JsonObject->GetStringField(TEXT("error")));
				return;
			}
			if (JsonObject->HasField(TEXT("comment")))
			{
				FReviewComment Comment = FReviewComment::FromJson(JsonObject->GetObjectField(TEXT("comment")));
				TakeMetadataFromBody(Comment);
				OnComplete.ExecuteIfBound(Comment, {});
				return;
			}
		}
		OnComplete.ExecuteIfBound({}, TEXT("Malformed Response"));
	});
	
	HttpRequest->ProcessRequest();
}

void FSwarmCommentsAPI::GetReviewTopicForCL(const FString& ChangelistNum,
	const OnGetReviewTopicForCLComplete& OnComplete) const
{
	const TSharedRef<IHttpRequest> HttpRequest = FHttpModule::Get().CreateRequest();
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json; charset=utf-8"));
	HttpRequest->SetHeader(TEXT("Authorization"), AuthTicket);
	
	HttpRequest->SetURL(FString::Format(TEXT("{0}?change={1}&max={2}"),
		{
			ReviewsURL(),
			ChangelistNum, // get the review for a specific CL
			1 // we only want a single review.
		}));
	HttpRequest->SetVerb(Keywords::HttpGet);

	TWeakPtr<const FSwarmCommentsAPI> WeakSelf = AsWeak();
	
	HttpRequest->OnProcessRequestComplete().BindLambda([OnComplete, WeakSelf](FHttpRequestPtr Request, FHttpResponsePtr Response, bool bConnectedSuccessfully)
	{
		const TSharedPtr<const FSwarmCommentsAPI> Self = WeakSelf.Pin();
		if (!Self)
		{
			return;
		}
		
		if (!bConnectedSuccessfully)
		{
			OnComplete.ExecuteIfBound({}, TEXT("Connection Failed"));
			return;
		}
		const TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(Response->GetContentAsString());
		TSharedPtr<FJsonObject> JsonObject;
		if (FJsonSerializer::Deserialize(JsonReader, JsonObject))
		{
			if (JsonObject->HasField(TEXT("error")))
			{
				OnComplete.ExecuteIfBound({}, JsonObject->GetStringField(TEXT("error")));
				return;
			}
			if (JsonObject->HasField(TEXT("reviews")))
			{
				const TArray<TSharedPtr<FJsonValue>> ReviewsJson = JsonObject->GetArrayField(TEXT("reviews"));
				if (ReviewsJson.IsEmpty())
				{
					OnComplete.ExecuteIfBound({}, TEXT("Review Not Found"));
					return;
				}
				
				const TSharedPtr<FJsonObject> Review = ReviewsJson[0]->AsObject();
				OnComplete.ExecuteIfBound(FReviewTopic{
					FString::FromInt(Review->GetIntegerField(TEXT("id"))),
					EReviewTopicType::Review
				}, {});
				return;
			}
		}
		
		OnComplete.ExecuteIfBound({}, TEXT("Malformed Response"));
	});
	
	HttpRequest->ProcessRequest();
}

void FSwarmCommentsAPI::GetOrCreateReviewTopicForCL(const FString& ChangelistNum,
	const OnGetReviewTopicForCLComplete& OnComplete) const
{
	TWeakPtr<const FSwarmCommentsAPI> WeakSelf = AsWeak();
	GetReviewTopicForCL(ChangelistNum,
		OnGetReviewTopicForCLComplete::CreateLambda([OnComplete, ChangelistNum, WeakSelf](const FReviewTopic& Topic, const FString& ErrorMessage)
		{
			const TSharedPtr<const FSwarmCommentsAPI> Self = WeakSelf.Pin();
			if (!Self)
			{
				return;
			}
			
			if (ErrorMessage == TEXT("Review Not Found"))
			{
				Self->CreateReviewTopicForCL(ChangelistNum, OnComplete);
			}
			else
			{
				OnComplete.Execute(Topic, ErrorMessage);
			}
		})
	);
}

void FSwarmCommentsAPI::CreateReviewTopicForCL(const FString& ChangelistNum, const OnGetReviewTopicForCLComplete& OnComplete) const
{
	
	const TSharedRef<IHttpRequest> HttpRequest = FHttpModule::Get().CreateRequest();
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json; charset=utf-8"));
	HttpRequest->SetHeader(TEXT("Authorization"), AuthTicket);
	
	HttpRequest->SetURL(ReviewsURL());
	HttpRequest->SetContentAsString(FString::Format(TEXT("{\"change\":{0}}"), {ChangelistNum}));
	HttpRequest->SetVerb(Keywords::HttpPost);
	
	HttpRequest->OnProcessRequestComplete().BindLambda([OnComplete](FHttpRequestPtr Request, FHttpResponsePtr Response, bool bConnectedSuccessfully)
	{
		if (!bConnectedSuccessfully)
		{
			OnComplete.ExecuteIfBound({}, TEXT("Connection Failed"));
			return;
		}
		const TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(Response->GetContentAsString());
		TSharedPtr<FJsonObject> JsonObject;
		if (FJsonSerializer::Deserialize(JsonReader, JsonObject))
		{
			if (JsonObject->HasField(TEXT("error")))
			{
				OnComplete.ExecuteIfBound({}, JsonObject->GetStringField(TEXT("error")));
				return;
			}
			if (JsonObject->HasField(TEXT("review")))
			{
				const TSharedPtr<FJsonObject> ReviewJson = JsonObject->GetObjectField(TEXT("review"));
				
				OnComplete.ExecuteIfBound(FReviewTopic{
					FString::FromInt(ReviewJson->GetIntegerField(TEXT("id"))),
					EReviewTopicType::Review
				}, {});
				return;
			}
		}
		
		OnComplete.ExecuteIfBound({}, TEXT("Malformed Response"));
	});
	
	HttpRequest->ProcessRequest();
}

FString FSwarmCommentsAPI::CommentsURL() const
{
	return SwarmURL.IsEmpty() ? FString{} : SwarmURL / TEXT("api/v9/comments");
}

FString FSwarmCommentsAPI::ReviewsURL() const
{
	return SwarmURL.IsEmpty() ? FString{} : SwarmURL / TEXT("api/v9/reviews");
}

void FSwarmCommentsAPI::GetUserdata(const FReviewTopic& Topic, const OnGetUserData& OnComplete) const
{
	if (FSwarmUserdataComment* Found = UserdataCache.Find(Topic))
	{
		OnComplete.ExecuteIfBound(Found, {});
		return;
	}

	GetComments(Topic, OnGetCommentsComplete::CreateLambda(
		[OnComplete, Topic, WeakSelf = AsWeak()](const TArray<FReviewComment>& Comments, const FString& ErrorMessage)
		{
			if (!ErrorMessage.IsEmpty())
			{
				OnComplete.ExecuteIfBound(nullptr, ErrorMessage);
				return;
			}
			const TSharedPtr<const FSwarmCommentsAPI> Self = WeakSelf.Pin();
			if (!Self)
			{
				return;
			}
			
			// GetComments should've set the UserdataCommentCache
			if (FSwarmUserdataComment* Found = Self->UserdataCache.Find(Topic))
			{
				OnComplete.ExecuteIfBound(Found, {});
				return;
			}
		}
	));
}

void FSwarmCommentsAPI::PostNewUserdataComment(const FReviewTopic& Topic) const
{
	FSwarmUserdataComment UserDataComment(GetUsername(), Topic);
			
	PostComment(UserDataComment, OnPostCommentComplete::CreateLambda(
		[WeakSelf = AsWeak()](const FReviewComment& Comment, const FString& ErrorMessage)
		{
			if (!ErrorMessage.IsEmpty())
			{
				UE_LOGF(LogSourceControl, Error, "FSwarmCommentsAPI::PostComment Error: %ls", *ErrorMessage)
				return;
			}
			if (const TSharedPtr<const FSwarmCommentsAPI> Self = WeakSelf.Pin())
			{
				// archive the comment so it's hidden in swarm
				FReviewComment Edit = Comment;
				Edit.bIsClosed = true;
				Edit.TaskState.Reset();
				Self->EditComment(Edit, OnEditCommentComplete::CreateLambda(
					[WeakSelf](const FReviewComment& Comment, const FString& ErrorMessage)
					{
						if (!ErrorMessage.IsEmpty())
						{
							UE_LOGF(LogSourceControl, Error, "FSwarmCommentsAPI::EditComment Error: %ls", *ErrorMessage)
							return;
						}
						if (const TSharedPtr<const FSwarmCommentsAPI> Self = WeakSelf.Pin())
						{
							Self->UserdataCache.Add(*Comment.Topic, Comment);
						}
					}), true);
			}
		}), true);
}

void FSwarmCommentsAPI::UpdateUserdata(const FReviewComment& ReviewComment) const
{
	// unset fields shouldn't change
	if (!ReviewComment.Likes.IsSet() && !ReviewComment.ReadBy.IsSet())
	{
		return;
	}
	
	GetUserdata(*ReviewComment.Topic, OnGetUserData::CreateLambda(
		[ReviewComment, WeakSelf = AsWeak()](FSwarmUserdataComment* UserDataComment, const FString& ErrorMessage)
		{
			if (!ErrorMessage.IsEmpty())
			{
				UE_LOGF(LogSourceControl, Error, "FSwarmCommentsAPI::UpdateUserdata Error: %ls", *ErrorMessage)
				return;
			}
			const TSharedPtr<const FSwarmCommentsAPI> Self = WeakSelf.Pin();
			if (!Self)
			{
				return;
			}

			bool bUserDataChanged = false;
			
			if (ReviewComment.Likes.IsSet())
			{
				bUserDataChanged |= UserDataComment->ReflectCommentLikes(ReviewComment);
			}
			if (ReviewComment.ReadBy.IsSet())
			{
				bUserDataChanged |= UserDataComment->ReflectCommentReads(ReviewComment);
			}
			if (bUserDataChanged)
			{
				Self->EditComment(*UserDataComment, nullptr, true);
			}
		}));
}

TWeakPtr<const FSwarmCommentsAPI> FSwarmCommentsAPI::AsWeak() const
{
	return StaticCastSharedRef<const FSwarmCommentsAPI>(AsShared()).ToWeakPtr();
}

TWeakPtr<FSwarmCommentsAPI> FSwarmCommentsAPI::AsWeak()
{
	return StaticCastSharedRef<FSwarmCommentsAPI>(AsShared()).ToWeakPtr();
}

void FSwarmCommentsAPI::PutMetadataInBody(FReviewComment& Comment)
{
	TMap<FString, FString> Metadata = ParseReviewCommentMetadata(Comment);
	if (Comment.Context.File.IsSet())
	{
		Metadata.FindOrAdd(Keywords::MetadataFile, *Comment.Context.File);
		Comment.Context.File.Reset();
	}
	if (Comment.Context.ReplyTo.IsSet())
	{
		Metadata.FindOrAdd(Keywords::MetadataReplyTo, FString::FromInt(*Comment.Context.ReplyTo));
		Comment.Context.ReplyTo.Reset();
	}
	if (Comment.Context.Category.IsSet())
	{
		Metadata.FindOrAdd(Keywords::MetadataCategory, *Comment.Context.Category);
		Comment.Context.Category.Reset();
	}
	SetReviewCommentMetadata(Comment, Metadata);
}

void FSwarmCommentsAPI::TakeMetadataFromBody(FReviewComment& Comment)
{
	// because swarm has very limited support for certain properties, they may be stored inside the body
	// as a workaround. parse them out and put them in their proper place.
	TMap<FString, FString> Metadata = ParseReviewCommentMetadata(Comment);
	if (const FString* File = Metadata.Find(Keywords::MetadataFile))
	{
		Comment.Context.File = *File;
	}
	if (const FString* ReplyTo = Metadata.Find(Keywords::MetadataReplyTo))
	{
		Comment.Context.ReplyTo = FCString::Atoi(**ReplyTo);
	}
	if (const FString* Category = Metadata.Find(Keywords::MetadataCategory))
	{
		Comment.Context.Category = *Category;
	}
	Comment.Body = Metadata[Keywords::MetadataBody];
}


static FString GetEnvironmentVariable(const char* Key)
{
	#pragma warning(suppress : 4996) // because I'm immediately copying the result of getenv() into managed memory, it's safe.
	if (const char* PathCStr = std::getenv(Key))
	{
		FString Result = FString(PathCStr);
		FPaths::NormalizeFilename(Result);
		return MoveTemp(Result);
	}
	return {};
}

// retrieves the default directory that p4tickets file is stored in
static FString GetP4TicketsPath()
{
	// if the P4TICKETS environment var is set, use that path
	const FString P4TicketsVar = GetEnvironmentVariable("P4TICKETS");
	if (!P4TicketsVar.IsEmpty())
	{
		return P4TicketsVar;
	}

	// if P4TICKETS wasn't set, default to "%USERPROFILE%\p4tickets.txt" on Windows, or "$HOME/.p4tickets" otherwise
#if PLATFORM_WINDOWS
	return GetEnvironmentVariable("USERPROFILE") / TEXT("p4tickets.txt");
#else
	return GetEnvironmentVariable("HOME") / TEXT(".p4tickets");
#endif
}

FSwarmCommentsAPI::FAuthTicket FSwarmCommentsAPI::RetrieveAuthorizationTicket()
{
	const ISourceControlProvider& SCCProvider = ISourceControlModule::Get().GetProvider();
	TMap<ISourceControlProvider::EStatus, FString> SCCStatus = SCCProvider.GetStatus();
	const FString* Username = SCCStatus.Find(ISourceControlProvider::EStatus::User);
	if (!Username)
	{
		return {};
	}
	
	TArray<FString> TicketStrings;
	FFileHelper::LoadFileToStringArray(TicketStrings, *GetP4TicketsPath());
	if (TicketStrings.IsEmpty())
	{
		return {};
	}

	TArray<FStringView> Options;
	for (FStringView TicketString : TicketStrings)
	{
		// find beginning of ticket
		int32 ChopIndex;
		if (!TicketString.FindChar('=', ChopIndex))
		{
			continue;
		}

		// remove the '='
		ChopIndex += 1;
		
		const FStringView Ticket = TicketString.RightChop(ChopIndex);
		if (Ticket.StartsWith(*Username + TEXT(":"), ESearchCase::IgnoreCase))
		{
			if (TicketString.StartsWith(TEXT("localhost")))
			{
				return {Ticket}; // prioritize localhost if the username matches
			}
			Options.Add(Ticket);
		}
	}
	if (!Options.IsEmpty())
	{
		if (Options.Num() > 1)
		{
			UE_LOGF(LogSourceControl, Warning, "Multiple viable tickets found for p4 user. Selecting one arbitrarily");
		}
		return {Options.Last()};
	}
	
	return {};
}


FString FSwarmCommentsAPI::RetrieveSwarmURL(const FString& Username)
{
	// Initialize P4Client
	Error P4Error;
	ClientApi P4Client;
	P4Client.Init(&P4Error);
	if (P4Error.Test())
	{
		StrBuf ErrorMsg;
		P4Error.Fmt(&ErrorMsg);
		UE_LOGF(LogSourceControl, Error, "P4ERROR: Invalid connection to server.");
		UE_LOGF(LogSourceControl, Error, "%ls", ANSI_TO_TCHAR(ErrorMsg.Text()));
		return {};
	}

	// Create a ClientUser that can capture the output from the command
	struct MyClientUser : ClientUser
	{
		virtual void OutputInfo(char level, const char* data) override
		{
			const FString Info(data);
			const int32 Found = Info.Find(TEXT(" = "));
			if (Found != INDEX_NONE)
			{
				Result = Info.RightChop(Found + 3);
			}
		}
		
		virtual void HandleError(Error* P4Error) override
		{
			StrBuf ErrorMsg;
			P4Error->Fmt(&ErrorMsg);
			UE_LOGF(LogSourceControl, Error, "P4ERROR: %ls", (ANSI_TO_TCHAR(ErrorMsg.Text())));
		}

		FString Result;
	} P4User;

	// Run property -l -n P4.Swarm.URL
	const char* ArgV[] = { "-l", "-n", "P4.Swarm.URL" };
	P4Client.SetArgv(3, const_cast<char* const*>(ArgV));
	P4Client.SetUser(TCHAR_TO_ANSI(*Username));
	P4Client.Run("property", &P4User);

	// Cleanup P4Client
	P4Client.Final(&P4Error);
	if (P4Error.Test())
	{
		StrBuf ErrorMsg;
		P4Error.Fmt(&ErrorMsg);
		UE_LOGF(LogSourceControl, Error, "P4ERROR: Failed to disconnect from Server.");
		UE_LOGF(LogSourceControl, Error, "%ls", (ANSI_TO_TCHAR(ErrorMsg.Text())));
		return {};
	}

	return P4User.Result;
}

FSwarmCommentsAPI::FAuthTicket::FAuthTicket(const FString& InUsername, const FString& InPassword)
	: Username(InUsername), Password(InPassword)
{}

FSwarmCommentsAPI::FAuthTicket::FAuthTicket(FStringView TicketString)
{
	int32 ChopIndex;
	if (TicketString.FindChar(':', ChopIndex))
	{
		Username = TicketString.Left(ChopIndex);
		Password = TicketString.RightChop(ChopIndex + 1);
	}
}

FSwarmCommentsAPI::FAuthTicket::operator FString() const
{
	return TEXT("Basic ") + FBase64::Encode(Username + TEXT(":") + Password);
}

bool FSwarmCommentsAPI::FAuthTicket::IsValid() const
{
	return !Username.IsEmpty() && !Password.IsEmpty();
}

FSwarmUserdataComment::FSwarmUserdataComment(FString User, FReviewTopic Topic)
{
	UserDataComment.User = User;
	UserDataComment.Topic = Topic;
	DataMap.Add(Keywords::UserdataLikedComments, {});
	DataMap.Add(Keywords::UserdataReadComments, {});
}

FSwarmUserdataComment::FSwarmUserdataComment(const FReviewComment& InUserDataComment)
	: UserDataComment(InUserDataComment)
	, DataMap(ParseReviewCommentMetadata(UserDataComment, Keywords::UserdataTag))
{
	// reset irrelevant fields
	UserDataComment.Likes.Reset();
	UserDataComment.ReadBy.Reset();
	UserDataComment.TaskState.Reset();
}

FSwarmUserdataComment::operator FReviewComment&()
{
	SetReviewCommentMetadata(UserDataComment, DataMap, Keywords::UserdataTag);
	return UserDataComment;
}

bool FSwarmUserdataComment::ReflectCommentLikes(const FReviewComment& ReviewComment)
{
	const FString& Username = *UserDataComment.User;
	const FString CommentID = FString::FromInt(*ReviewComment.CommentID);
	const EUserdataSetToggleVal ToggleVal = ReviewComment.Likes->Find(Username)? EUserdataSetToggleVal::ToggleSet : EUserdataSetToggleVal::ToggleUnset;
	return ToggleSetItem(Keywords::UserdataLikedComments, CommentID, ToggleVal);
}

bool FSwarmUserdataComment::ReflectCommentReads(const FReviewComment& ReviewComment)
{
	const FString& Username = *UserDataComment.User;
	const FString CommentID = FString::FromInt(*ReviewComment.CommentID);
	const EUserdataSetToggleVal ToggleVal = ReviewComment.ReadBy->Find(Username)? EUserdataSetToggleVal::ToggleSet : EUserdataSetToggleVal::ToggleUnset;
	return ToggleSetItem(Keywords::UserdataReadComments, CommentID, ToggleVal);
}

TSet<FString> FSwarmUserdataComment::GetLikedComments() const
{
	return DeserializeSetString(DataMap[Keywords::UserdataLikedComments]);
}

TSet<FString> FSwarmUserdataComment::GetReadComments() const
{
	return DeserializeSetString(DataMap[Keywords::UserdataReadComments]);
}

bool FSwarmUserdataComment::IsUserdataComment(const FReviewComment& Comment)
{
	if (!Comment.Body.IsSet())
	{
		return false;
	}
	return Comment.Body->Find(Keywords::UserdataTag) != INDEX_NONE;
}

bool FSwarmUserdataComment::ToggleSetItem(const FString& SetName, const FString& SetKey, EUserdataSetToggleVal ToggleVal)
{
	TSet<FString> Set = DeserializeSetString(DataMap[SetName]);
	switch(ToggleVal)
	{
	case EUserdataSetToggleVal::ToggleSet:
		Set.FindOrAdd(SetKey);
		break;
	case EUserdataSetToggleVal::ToggleUnset:
		Set.Remove(SetKey);
		break;
	}

	const FString NewSetString = SerializeSetString(Set);
	if (DataMap[SetName] != NewSetString)
	{
		DataMap[SetName] = NewSetString;
		SetReviewCommentMetadata(UserDataComment, DataMap, Keywords::UserdataTag);
		return true;
	}
	return false;
}
