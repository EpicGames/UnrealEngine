// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ReviewComments.h"

// helper class that stores likes and Read comments as an archived comment in a review
class FSwarmUserdataComment
{
public:
	FSwarmUserdataComment(FString User, FReviewTopic Topic);
	FSwarmUserdataComment(const FReviewComment& InUserDataComment);
	operator FReviewComment&();
	bool ReflectCommentLikes(const FReviewComment& ReviewComment);
	bool ReflectCommentReads(const FReviewComment& ReviewComment);
	TSet<FString> GetLikedComments() const;
	TSet<FString> GetReadComments() const;
	
	static bool IsUserdataComment(const FReviewComment& Comment);
private:
	enum class EUserdataSetToggleVal { ToggleSet, ToggleUnset };
	bool ToggleSetItem(const FString& SetName, const FString& SetKey, EUserdataSetToggleVal ToggleVal);
	FReviewComment UserDataComment;
	TMap<FString,FString> DataMap;
};

class FSwarmCommentsAPI : public IReviewCommentAPI
{
	struct FAuthTicket;
public:
	// prefer TryConnect over calling constructor directly
	FSwarmCommentsAPI(const FString& AuthTicket, const FString& SwarmURL);
	FSwarmCommentsAPI(const FAuthTicket& AuthTicket, const FString& SwarmURL);
	static TSharedPtr<FSwarmCommentsAPI> TryConnect();

	virtual FString GetUsername() const override;

	virtual void GetComments(const FReviewTopic& Topic, const OnGetCommentsComplete& OnComplete) const override;
	// Body must be set
	virtual void PostComment(FReviewComment& Comment, const OnPostCommentComplete& OnComplete, bool bSilenceNotification = false) const override;
	// all unset members will be left unchanged.
	virtual void EditComment(const FReviewComment& Comment, const OnEditCommentComplete& OnComplete, bool bSilenceNotification = false) const override;
	
	// retrieves the review associated with a CL.
	virtual void GetReviewTopicForCL(const FString &ChangelistNum, const OnGetReviewTopicForCLComplete& OnComplete) const override;
	
	// retrieves the review associated with a CL. If there is none, it creates a new review and returns that.
	virtual void GetOrCreateReviewTopicForCL(const FString& ChangelistNum, const OnGetReviewTopicForCLComplete& OnComplete) const override;

private:
	
	void CreateReviewTopicForCL(const FString &ChangelistNum, const OnGetReviewTopicForCLComplete& OnComplete) const;

	FString CommentsURL() const;
	FString ReviewsURL() const;

	DECLARE_DELEGATE_TwoParams(OnGetUserData, FSwarmUserdataComment*, const FString& /*ErrorMessage*/)
	void GetUserdata(const FReviewTopic& Topic, const OnGetUserData& OnComplete) const;
	void PostNewUserdataComment(const FReviewTopic& Topic) const;
	void UpdateUserdata(const FReviewComment& ReviewComment) const;

	TWeakPtr<const FSwarmCommentsAPI> AsWeak() const;
	TWeakPtr<FSwarmCommentsAPI> AsWeak();
	
	static void PutMetadataInBody(FReviewComment& Comment);
	static void TakeMetadataFromBody(FReviewComment& Comment);
	
	static FAuthTicket RetrieveAuthorizationTicket();
	static FString RetrieveSwarmURL(const FString& Username);

	// used to authorize http requests to swarm
	struct FAuthTicket
	{
		FAuthTicket() = default;
		FAuthTicket(const FString& Username, const FString& Password);
		FAuthTicket(FStringView TicketString);
		operator FString() const;
		bool IsValid() const;
		FString Username;
	private:
		FString Password;
	} AuthTicket;

	// base url for all swarm api requests
	FString SwarmURL;

	// maps user to the comment that stores the users data like their likes and views
	// this is necessary because the swarm api unfortunately has incomplete support for updating like and view state
	mutable TMap<FReviewTopic, FSwarmUserdataComment> UserdataCache;
};
