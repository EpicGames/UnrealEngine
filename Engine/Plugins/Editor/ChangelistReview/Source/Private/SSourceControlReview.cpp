// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSourceControlReview.h"

#include "AssetRegistry/AssetData.h"
#include "ClassIconFinder.h"
#include "FSwarmCommentsAPI.h"
#include "Framework/Views/TableViewMetadata.h"
#include "ISourceControlModule.h"
#include "SourceControlOperations.h"
#include "SSourceControlReviewEntry.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Styling/StarshipCoreStyle.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Input/SButton.h"
#include "Misc/App.h"
#include "Misc/MessageDialog.h"
#include "Styling/StyleColors.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Internationalization/Regex.h"
#include "Widgets/Input/SComboBox.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/LinkerLoad.h"

#define LOCTEXT_NAMESPACE "SourceControlReview"

using namespace SourceControlReview;

namespace UE::DiffControl
{
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnCommentPosted, const FReviewComment&)
	extern KISMET_API FOnCommentPosted GOnCommentPosted;
}


namespace ReviewHelpers
{
	const FString FileDepotKey = TEXT("depotFile");
	const FString FileRevisionKey = TEXT("rev");
	const FString FileActionKey = TEXT("action");
	const FString TimeKey = TEXT("time");
	const FString AuthorKey = TEXT("user");
	const FString DescriptionKey = TEXT("desc");
	const FString ChangelistStatusKey = TEXT("status");
	const FString ChangelistPendingStatusKey = TEXT("pending");
	constexpr int32 RecordIndex = 0;
}

const UClass* FChangelistFileData::GetIconClass()
{
	// If we haven't cached the icon class yet, find it
	if (!CachedIconClass.IsSet())
	{
		CachedIconClass = nullptr;
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		const FString TempPackageName = FPackagePath::FromLocalPath(ReviewFileTempPath).GetPackageName();
		if (!TempPackageName.IsEmpty())
		{
			TArray<FAssetData> OutAssetData;
			AssetRegistryModule.Get().GetAssetsByPackageName(*TempPackageName, OutAssetData);

			if (OutAssetData.Num() > 0)
			{
				CachedIconClass = FClassIconFinder::GetIconClassForAssetData(OutAssetData[0]);
			}
		}
	}

	return CachedIconClass.GetValue();
}

void SSourceControlReview::Construct(const FArguments& InArgs)
{
	const FString ProjectName = FApp::GetProjectName();
	ensureAlwaysMsgf((!ProjectName.IsEmpty()), TEXT("BlueprintReviewTool - Unable to get ProjectName"));

	const static FSlateRoundedBoxBrush RecessedBrush(FStyleColors::Recessed, CoreStyleConstants::InputFocusRadius);
	const static FEditableTextBoxStyle InfoWidgetStyle =
		FEditableTextBoxStyle(FAppStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"))
		.SetBackgroundImageNormal(RecessedBrush)
		.SetBackgroundImageHovered(RecessedBrush)
		.SetBackgroundImageFocused(RecessedBrush)
		.SetBackgroundImageReadOnly(RecessedBrush);

	const static FMargin InfoWidgetMargin(4.f, 2.f, 4.f, 8.f);

	CommentsAPI = FSwarmCommentsAPI::TryConnect();
	
	LoadCLHistory();
	
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Brushes.Panel"))
		.Padding(10.f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(ChangelistInfoWidget, SGridPanel)
				.FillColumn(1, 1.f)

				// Changelist
				+SGridPanel::Slot(0,0)
				.Padding(InfoWidgetMargin)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ChangelistNumber", "Changelist"))
					.Font(FStyleFonts::Get().Normal)
				]
				+SGridPanel::Slot(1,0)
				.Padding(4.f, 0.f, 4.f, 8.f)
				.HAlign(HAlign_Left)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(0.f, 0.f, 8.f, 0.f)
					.AutoWidth()
					[
						SAssignNew(ChangelistNumComboBox, SComboBox<TSharedPtr<FChangelistLightInfo>>)
						.OptionsSource(&CLHistory)
                     	.IsEnabled(this, &SSourceControlReview::IsSourceControlActive)
						.OnGenerateWidget(this, &SSourceControlReview::MakeCLComboOption)
						.OnSelectionChanged(this, &SSourceControlReview::OnCLComboSelection)
						.ContentPadding(0.f)
						.Content()
						[
							SAssignNew(ChangelistNumText, SEditableText)
                     		.Font(FStyleFonts::Get().Normal)
                     		.MinDesiredWidth(55.f)
                     		.Justification(ETextJustify::Center)
                     		.IsEnabled(this, &SSourceControlReview::IsSourceControlActive)
                     		.OnTextCommitted(this, &SSourceControlReview::OnChangelistNumCommitted)
                     		.OnTextChanged(this, &SSourceControlReview::OnChangelistNumChanged)
						]
						
					]
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.IsEnabled(this, &SSourceControlReview::IsSourceControlActive)
						.OnClicked(this, &SSourceControlReview::OnLoadChangelistClicked)
						.ToolTipText(this, &SSourceControlReview::LoadChangelistTooltip)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("LoadChangelistText", "Load"))
							.Font(FStyleFonts::Get().Normal)
						]
					]
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.Visibility(this, &SSourceControlReview::EnableCommentsButtonVisibility)
						.OnClicked(this, &SSourceControlReview::OnEnableCommentsButtonClicked)
						.ToolTipText(LOCTEXT("EnableCommentsTooltip", "Create a swarm review to store comments"))
						[
							SNew(STextBlock)
							.Text(LOCTEXT("EnableComments", "Enable Comments"))
							.Font(FStyleFonts::Get().Normal)
						]
					]
				]

				// Author
				+SGridPanel::Slot(0,1)
				.Padding(InfoWidgetMargin)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ChangelistAuthor", "Author"))
					.Font(FStyleFonts::Get().Normal)
				]
				+SGridPanel::Slot(1,1)
				.Padding(4.f, 0.f, 4.f, 8.f)
				.HAlign(HAlign_Fill)
				[
					SNew(SEditableTextBox)
					.Text_Lambda([this]{return CurrentChangelistInfo.Author;})
					.IsReadOnly(true)
					.Style(&InfoWidgetStyle)
				]

				// Path
				+SGridPanel::Slot(0,2)
				.Padding(InfoWidgetMargin)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ChangelistPath", "Path"))
					.Font(FStyleFonts::Get().Normal)
				]
				+SGridPanel::Slot(1,2)
				.Padding(4.f, 0.f, 4.f, 8.f)
				.HAlign(HAlign_Fill)
				[
					SNew(SEditableTextBox)
					.Text_Lambda([this]
					{
						return CurrentChangelistInfo.SharedPath.IsEmpty() ? FText::GetEmpty() : FText::Format(LOCTEXT("CLPath", "{0}/..."), CurrentChangelistInfo.SharedPath);
					})
					.IsReadOnly(true)
					.Style(&InfoWidgetStyle)
				]

				// Status
				+SGridPanel::Slot(0,3)
				.Padding(InfoWidgetMargin)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ChangelistStatus", "Status"))
					.Font(FStyleFonts::Get().Normal)
				]
				+SGridPanel::Slot(1,3)
				.Padding(4.f, 0.f, 4.f, 8.f)
				.HAlign(HAlign_Fill)
				[
					SNew(SEditableTextBox)
					.Text_Lambda([this]{return CurrentChangelistInfo.Status;})
					.IsReadOnly(true)
					.Style(&InfoWidgetStyle)
				]

				// Description
				+SGridPanel::Slot(0,4)
				.Padding(InfoWidgetMargin)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ChangelistDescription", "Description"))
					.Font(FStyleFonts::Get().Normal)
				]
				+SGridPanel::Slot(1,4)
				.Padding(InfoWidgetMargin)
				.HAlign(HAlign_Fill)
				[
					SNew(SBox)
					.MaxDesiredHeight(147.f)
					.MinDesiredHeight(147.f)
					[
						SNew(SMultiLineEditableTextBox)
						.Text_Lambda([this]{return CurrentChangelistInfo.Description;})
						.AutoWrapText(true)
						.IsReadOnly(true)
						.Style(&InfoWidgetStyle)
					]
				]
			]
			
			+SVerticalBox::Slot()
			.VAlign(VAlign_Top)
			[
				SAssignNew(ChangelistEntriesWidget, SListView<TSharedPtr<FChangelistFileData>>)
				.ListItemsSource(&ChangelistFiles)
				.OnGenerateRow(this, &SSourceControlReview::OnGenerateFileRow)
				.HeaderRow(
					SNew(SHeaderRow)
					+HeaderColumn(ColumnIds::Status)
					+HeaderColumn(ColumnIds::File)
					+HeaderColumn(ColumnIds::Tools)
				)
			]
			
			+SVerticalBox::Slot()
			.Padding(0.f, 0.f, 0.f, 98.f)
			.HAlign(HAlign_Center)
			.AutoHeight()
			[
				SAssignNew(EnterChangelistTextBlock, STextBlock)
				.Visibility(EVisibility::Visible)
				.ColorAndOpacity(FStyleColors::AccentGray)
				.Text(LOCTEXT("EnterChangelistText", "Enter a Changelist number above to search"))
			]
			
			+SVerticalBox::Slot()
			.Padding(0.f, 10.f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Bottom)
			.AutoHeight()
			[
				SAssignNew(LoadingTextBlock, STextBlock)
				.Visibility(EVisibility::Collapsed)
				.Text(LOCTEXT("LoadingText", "Loading..."))
				.Font(FStyleFonts::Get().Large)
			]
			
			+SVerticalBox::Slot()
			.Padding(0.f, 15.f)
			.AutoHeight()
			[
				SAssignNew(LoadingProgressBar, SProgressBar)
				.Visibility(EVisibility::Collapsed)
				.Percent(1.f)
			]
		]
	];
}

SSourceControlReview::~SSourceControlReview()
{
	if (GetChangelistDetailsCommand)
	{
		ISourceControlModule::Get().GetProvider().CancelOperation(GetChangelistDetailsCommand.ToSharedRef());
		GetChangelistDetailsCommand.Reset();
	}
}

void SSourceControlReview::LoadChangelist(const FString& Changelist)
{
	if (!IsLoading())
	{
		SetLoading(true);
	}
	else
	{
		FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("ReviewChangelistTool", "ChangelistError", "Changelist is already loading"));
		return;
	}

	ChangelistFiles.Empty();
	GlobalReviewComments.Empty();
	FileReviewComments.Empty();
	TempLocalPathToDepotPath.Empty();
	ReviewTopic.Reset();

	//This command runs p4 -describe (or similar for other version controls) to retrieve changelist record information
	GetChangelistDetailsCommand = ISourceControlOperation::Create<FGetChangelistDetails>();
	GetChangelistDetailsCommand->SetChangelistNumber(Changelist);

	ISourceControlModule::Get().GetProvider().Execute(
		GetChangelistDetailsCommand.ToSharedRef(),
		EConcurrency::Asynchronous,
		FSourceControlOperationComplete::CreateRaw(this, &SSourceControlReview::OnGetChangelistDetails, Changelist));
}

bool SSourceControlReview::OpenChangelist(const FString& Changelist)
{
	if (ChangelistNumText)
	{
		ChangelistNumText->SetText(FText::FromString(Changelist));
	}
	LoadChangelist(Changelist);
	return IsLoading();
}

const TArray<FReviewComment>* SSourceControlReview::GetReviewCommentsForFile(const FString& FilePath)
{
	return FileReviewComments.Find(AsDepotPath(FilePath));
}

void SSourceControlReview::UpdateReviewComments()
{
	if (bReviewCommentsDirty && CommentsAPI && ReviewTopic)
	{
		CommentsAPI->GetComments(ReviewTopic.GetValue(), FSwarmCommentsAPI::OnGetCommentsComplete::CreateRaw(this, &SSourceControlReview::OnGetReviewComments));
		bReviewCommentsDirty = false;
	}
}

void SSourceControlReview::PostComment(FReviewComment& Comment)
{
	if (!CommentsAPI)
	{
		return;
	}
	
	bReviewCommentsDirty = true;
	
	// convert filepath from local to depot path
	if (Comment.Context.File.IsSet())
	{
		FString& Path = Comment.Context.File.GetValue();
		Path = AsDepotPath(Path);
	}
	
	Comment.Topic = ReviewTopic;
	CommentsAPI->PostComment(Comment, IReviewCommentAPI::OnPostCommentComplete::CreateLambda(
		[](const FReviewComment& Comment, const FString& ErrorMessage)
		{
			if (!ErrorMessage.IsEmpty())
			{
				UE_LOGF(LogSourceControl, Error, "IReviewCommentsAPI::PostComment Error: %ls", *ErrorMessage)
				return;
			}
			// notify listeners that the comment successfully posted
			if (UE::DiffControl::GOnCommentPosted.IsBound())
			{
				UE::DiffControl::GOnCommentPosted.Broadcast(Comment);
			}
		}));
}

void SSourceControlReview::EditComment(FReviewComment& Comment)
{
	if (!CommentsAPI)
	{
		return;
	}
	
	bReviewCommentsDirty = true;
	
	// convert filepath from local to depot path
	if (Comment.Context.File.IsSet())
	{
		FString& Path = Comment.Context.File.GetValue();
		Path = AsDepotPath(Path);
	}
	
	Comment.Topic = ReviewTopic;
	CommentsAPI->EditComment(Comment, IReviewCommentAPI::OnEditCommentComplete::CreateLambda(
		[](const FReviewComment& Comment, const FString& ErrorMessage)
		{
			if (!ErrorMessage.IsEmpty())
			{
				UE_LOGF(LogSourceControl, Error, "IReviewCommentsAPI::EditComment Error: %ls", *ErrorMessage)
			}
		}));
}

FString SSourceControlReview::GetReviewerUsername() const
{
	return CommentsAPI? CommentsAPI->GetUsername() : FString{};
}

bool SSourceControlReview::IsFileInReview(const FString& File) const
{
	// require review comments to be valid
	if (!CommentsAPI || !ReviewTopic.IsSet() || IsLoading())
	{
		return false;
	}
	
	FString Path = File;
	if (!Path.StartsWith(TEXT("//")))
	{
		// if a local path was passed in, commentable paths are found in TempLocalPathToDepotPath
		Path = FPaths::ConvertRelativePathToFull(Path);
		FPaths::NormalizeFilename(Path);
		return TempLocalPathToDepotPath.Find(Path) != nullptr;
	}

	// if a depot path was passed in, we can early out if it's in FileReviewComments
	if (FileReviewComments.Find(Path))
	{
		return true;
	}

	// if the file hasn't been commented on before, it might still be in the review. fallback to searching ChangelistFiles
	for (const TSharedPtr<FChangelistFileData>& FileData : ChangelistFiles)
	{
		if (FileData->AssetDepotPath == Path)
		{
			return true;
		}
	}
	return false;
}

FString SSourceControlReview::AsDepotPath(const FString& FilePath)
{
	if (!FilePath.StartsWith(TEXT("//")))
	{
		FString LocalPath = FPaths::ConvertRelativePathToFull(FilePath);
		FPaths::NormalizeFilename(LocalPath);
		if (const FString* DepotPath = TempLocalPathToDepotPath.Find(LocalPath))
		{
			return *DepotPath;
		}
		return {};
	}
	return FilePath;
}

void SSourceControlReview::CommitTempChangelistNumToHistory()
{
	if (bUncommittedChangelistNum)
	{
		// bUncommittedChangelistNum should never be true when the history is empty
		check(!CLHistory.IsEmpty())
		
		// temp cl nums are missing an author and description. we have that now so let's provide it.
		// annoyingly the SComboBox doesn't visually update unless the shared pointer is different so we have to reallocate it :(
		CLHistory[0] = MakeShared<FChangelistLightInfo>(CLHistory[0]->Number, CurrentChangelistInfo.Author, CurrentChangelistInfo.Description);

		// make sure there's only one copy of this CL in the history.
		for(int32 I = 1; I < CLHistory.Num(); ++I)
		{
			if (CLHistory[I]->Number.EqualTo(CLHistory[0]->Number))
			{
				CLHistory.RemoveAt(I);
				break; // there should only be a max of 1 duplicate so we can break early
			}
		}
		ChangelistNumComboBox->RefreshOptions();
		bUncommittedChangelistNum = false;
	}
}

void SSourceControlReview::RemoveUncommittedChangelistNumFromHistory()
{
	if (bUncommittedChangelistNum)
	{
		// bUncommittedChangelistNum should never be true when the history is empty
		check(!CLHistory.IsEmpty())
		
		CLHistory.RemoveAt(0); 
		bUncommittedChangelistNum = false;
	}
}

void SSourceControlReview::IncrementItemsLoaded()
{
	NumItemsLoaded++;
	LoadingProgressBar->SetPercent(NumItemsToLoad ? static_cast<float>(NumItemsLoaded) / static_cast<float>(NumItemsToLoad) : 1.f);
	if (NumItemsToLoad == NumItemsLoaded)
	{
		OnLoadComplete();
	}
}

void SSourceControlReview::OnGetChangelistDetails(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult, FString Changelist)
{
	// this command is cancelled when this widget is destroyed. Exit immediately to avoid touching invalid data
	if (InResult == ECommandResult::Cancelled)
	{
		return;
	}
	
	const TSharedRef<FGetChangelistDetails, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FGetChangelistDetails>(InOperation);
	const TArray<TMap<FString, FString>>& Record = Operation->GetChangelistDetails();
	
	if (!IsChangelistRecordValid(Record))
	{
		RemoveUncommittedChangelistNumFromHistory();
		SetLoading(false);
		return;
	}

	const TMap<FString, FString>& ChangelistRecord = Record[ReviewHelpers::RecordIndex];
	
	//Num files we are going to expect retrieved from source control
	NumItemsToLoad = 0;
	//Num files we've loaded so far
	NumItemsLoaded = 0;

	// TODO: @jordan.hoffmann revise for other version controls
	//Each file in p4 has an index "depotFile0" this is the index that is updated by the while loop to find all the files "depotFile1", "depotFile2" etc...
	uint32  RecordFileIndex = 0;
	//String representation of the current file index
	FString RecordFileIndexStr = LexToString(RecordFileIndex);
	//The p4 records is the map a file key starts with "depotFile" and is followed by file index 
	FString RecordFileMapKey = ReviewHelpers::FileDepotKey + RecordFileIndexStr;
	//The p4 records is the map a revision key starts with "rev" and is followed by file index 
	FString RecordRevisionMapKey = ReviewHelpers::FileRevisionKey + RecordFileIndexStr;
	//The p4 records is the map a revision key starts with "action" and is followed by file index 
	FString RecordActionMapKey = ReviewHelpers::FileActionKey + RecordFileIndexStr;
	const bool bIsShelved = ChangelistRecord[ReviewHelpers::ChangelistStatusKey] == ReviewHelpers::ChangelistPendingStatusKey;

	SetChangelistInfo(ChangelistRecord, Changelist);
	CommitTempChangelistNumToHistory();
	SaveCLHistory();
	
	if (CommentsAPI)
	{
		NumItemsToLoad += 2; // require review topic and comments to be loaded before continuing
		if (ChangelistRecord[ReviewHelpers::AuthorKey] == TEXT("swarm"))
		{
			// if a swarm cl was provided, we can construct the review topic ourselves
			OnGetReviewTopic(FReviewTopic{Changelist, EReviewTopicType::Review}, {});
		}
		else
		{
			// if a swarm cl wasn't provided, request one.
			CommentsAPI->GetReviewTopicForCL(Changelist, FSwarmCommentsAPI::OnGetReviewTopicForCLComplete::CreateRaw(this, &SSourceControlReview::OnGetReviewTopic));
		}
	}
	
	//the loop checks if we have a valid record "depotFile(Index)" in the records to add a file entry
	while (ChangelistRecord.Contains(RecordFileMapKey) && ChangelistRecord.Contains(RecordRevisionMapKey))
	{
		const FString &FileDepotPath = ChangelistRecord[RecordFileMapKey];

		const int32 AssetRevision = FCString::Atoi(*ChangelistRecord[RecordRevisionMapKey]);
	
		TSharedPtr<FChangelistFileData> ChangelistFileData = MakeShared<FChangelistFileData>();

		ChangelistFileData->AssetName = FPaths::GetBaseFilename(FileDepotPath, true);
		ChangelistFileData->ReviewFileRevisionNum = ChangelistRecord[RecordRevisionMapKey];
		
		ChangelistFileData->ReviewFileDateTime = FDateTime(1970, 1, 1, 0, 0, 0, 0) + FTimespan::FromSeconds(FCString::Atoi(*ChangelistRecord[ReviewHelpers::TimeKey]));
		ChangelistFileData->ChangelistNum = FCString::Atoi(*Changelist);

		//Determine if we are dealing with submitted or pending changelist
		ChangelistFileData->ChangelistState = bIsShelved ? EChangelistState::Pending : EChangelistState::Submitted;

		//Building absolute local path is needed to use local file to retrieve file history information from p4 to then show file revision data
		ChangelistFileData->AssetDepotPath = FileDepotPath;
		ChangelistFileData->AssetFilePath = AsAssetPath(ChangelistFileData->AssetDepotPath);
		ChangelistFileData->RelativeFilePath = TrimSharedPath(ChangelistRecord[RecordFileMapKey]);

		SetFileSourceControlAction(ChangelistFileData, ChangelistRecord[RecordActionMapKey]);

		const int32 PreviousAssetRevision = bIsShelved && (ChangelistFileData->FileSourceControlAction == ESourceControlAction::Delete || ChangelistFileData->FileSourceControlAction == ESourceControlAction::Edit) ? AssetRevision : AssetRevision - 1;
		ChangelistFileData->PreviousFileRevisionNum = FString::FromInt(PreviousAssetRevision);
		
		TWeakPtr<SSourceControlReview> WeakReviewWidget = SharedThis(this);
		const auto GetFileCommandResponse = [WeakReviewWidget, ChangelistFileData](const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult, FString* OutFilePath)
		{
			if (WeakReviewWidget.IsValid())
			{
				const TSharedRef<FGetFile, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FGetFile>(InOperation);
				*OutFilePath = Operation->GetOutPackageFilename();
				WeakReviewWidget.Pin()->OnGetFileFromSourceControl(ChangelistFileData);
			}
		};
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

		// retrieve files directly from source control into a temp location
		if (ChangelistFileData->FileSourceControlAction != ESourceControlAction::Delete)
		{
			NumItemsToLoad++;
			
			TSharedRef<FGetFile> GetFileToReviewCommand = ISourceControlOperation::Create<FGetFile>(Changelist, ChangelistFileData->ReviewFileRevisionNum, ChangelistRecord[RecordFileMapKey], bIsShelved);

			SourceControlProvider.Execute(GetFileToReviewCommand, EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateLambda(GetFileCommandResponse, &ChangelistFileData->ReviewFileTempPath));
		}
		
		if (ChangelistFileData->FileSourceControlAction != ESourceControlAction::Add && ChangelistFileData->FileSourceControlAction != ESourceControlAction::Branch)
		{
			NumItemsToLoad++;

			// Request the following additional info about the previous revision:
			// - Date/Time of submission
			// - CL number
			FString PrevRevisionDepotPath = FString::Format(TEXT("{0}#{1}"), {ChangelistFileData->AssetDepotPath, ChangelistFileData->PreviousFileRevisionNum});
			TSharedRef<FGetSourceControlRevisionInfo> GetSourceControlRevisionInfo = ISourceControlOperation::Create<FGetSourceControlRevisionInfo>();
			SourceControlProvider.Execute(GetSourceControlRevisionInfo, PrevRevisionDepotPath, EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateLambda(
				[ChangelistFileData](const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
				{
					const TSharedRef<FGetSourceControlRevisionInfo> Operation = StaticCastSharedRef<FGetSourceControlRevisionInfo>(InOperation);
					ChangelistFileData->PreviousFileDateTime = Operation->GetDateTime();
					ChangelistFileData->PreviousChangelistNum = FCString::Atoi(*Operation->GetChangelist());
				}));

			// get the contents of the previous revision
			TSharedRef<FGetFile> GetPreviousFileCommand = ISourceControlOperation::Create<FGetFile>(Changelist, ChangelistFileData->PreviousFileRevisionNum, ChangelistRecord[RecordFileMapKey], false);

			SourceControlProvider.Execute(GetPreviousFileCommand, EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateLambda(GetFileCommandResponse, &ChangelistFileData->PreviousFileTempPath));
		}

		//Update map keys with the next file index
		//When we are doing looking at for example "depotFile0" we build our next key "depotFile1"
		RecordFileIndex++;
		RecordFileIndexStr = LexToString(RecordFileIndex);
		RecordFileMapKey = ReviewHelpers::FileDepotKey + RecordFileIndexStr;
		RecordRevisionMapKey = ReviewHelpers::FileRevisionKey + RecordFileIndexStr;
		RecordActionMapKey = ReviewHelpers::FileActionKey + RecordFileIndexStr;
	}

	//If we have no files to load flip loading bar visibility
	if (NumItemsToLoad == 0)
	{
		SetLoading(false);
	}
}


void SSourceControlReview::OnGetReviewTopic(const FReviewTopic& Topic, const FString& ErrorMessage)
{
	if(!ErrorMessage.IsEmpty())
	{
		// if a review wasn't found, we'll display a button to create a review
		if (ErrorMessage != TEXT("Review Not Found"))
		{
			UE_LOGF(LogSourceControl, Error, "IReviewCommentsAPI::GetReviewTopicForCL Error: %ls", *ErrorMessage);
		}
		
		// skip loading comments
		IncrementItemsLoaded();
		IncrementItemsLoaded();
		return;
	}
	else
	{
		ReviewTopic = Topic;
	}
	IncrementItemsLoaded();
	
	if (CommentsAPI)
	{
		CommentsAPI->GetComments(ReviewTopic.GetValue(), FSwarmCommentsAPI::OnGetCommentsComplete::CreateRaw(this, &SSourceControlReview::OnInitReviewComments));
	}
	
	// every 3 seconds, check whether comment data is dirty and if so, grab the update from swarm
	ChangelistInfoWidget.Get()->RegisterActiveTimer(3.f, FWidgetActiveTimerDelegate::CreateLambda([this](double, float)
	{
		UpdateReviewComments();
		return EActiveTimerReturnType::Continue;
	}));
}

void SSourceControlReview::OnGetReviewComments(const TArray<FReviewComment>& Comments, const FString& ErrorMessage)
{
	bReviewCommentsDirty = false;
	if (!ErrorMessage.IsEmpty())
	{
		UE_LOGF(LogSourceControl, Error, "IReviewCommentsAPI::GetComments Error: %ls", *ErrorMessage)
		return;
	}

	FileReviewComments.Empty();
	GlobalReviewComments.Empty();
	for (const FReviewComment& Comment : Comments)
	{
		if (Comment.Context.File.IsSet())
		{
			const FString& File = Comment.Context.File.GetValue();
			TArray<FReviewComment>& FileComments = FileReviewComments.FindOrAdd(File);
			FileComments.Add(Comment);
			continue;
		}
		GlobalReviewComments.Add(Comment);
	}
}

void SSourceControlReview::OnInitReviewComments(const TArray<FReviewComment>& Comments, const FString& ErrorMessage)
{
	OnGetReviewComments(Comments, ErrorMessage);
	IncrementItemsLoaded();
}

void SSourceControlReview::OnGetFileFromSourceControl(TSharedPtr<FChangelistFileData> ChangelistFileData)
{
	if (ChangelistFileData && ChangelistFileData->IsDataValidForEntry())
	{
		ChangelistFiles.Add(ChangelistFileData);
		if (!ChangelistFileData->ReviewFileTempPath.IsEmpty())
		{
			TempLocalPathToDepotPath.Add(ChangelistFileData->ReviewFileTempPath, ChangelistFileData->AssetDepotPath);
		}
		if (!ChangelistFileData->PreviousFileTempPath.IsEmpty())
		{
			TempLocalPathToDepotPath.Add(ChangelistFileData->PreviousFileTempPath, ChangelistFileData->AssetDepotPath);
		}
	}
	
	IncrementItemsLoaded();
}

void SSourceControlReview::OnLoadComplete()
{
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>( TEXT( "AssetRegistry" ) );

	// Merge asset moves and renames into single entries
	FixupRedirectors();
		
	Algo::Sort(ChangelistFiles, [](const TSharedPtr<FChangelistFileData> &A, const TSharedPtr<FChangelistFileData> &B)
	{
		return A->RelativeFilePath < B->RelativeFilePath;
	});
	
	TArray<FString> ChangelistFilePaths;
	for (const TSharedPtr<FChangelistFileData>& FileData : ChangelistFiles)
	{
		if(FileData)
		{
			ChangelistFilePaths.Add(FileData->ReviewFileTempPath);
		}
	}
	
	AssetRegistryModule.Get().ScanFilesSynchronous(ChangelistFilePaths);

	// now that the files are in the asset registry, cache their associated class so their class icons can be created quickly
	for (const TSharedPtr<FChangelistFileData>& FileData : ChangelistFiles)
	{
		if(FileData)
		{
			FileData->GetIconClass();
		}
	}
	
	ChangelistEntriesWidget->RebuildList();
	SetLoading(false);
}

FReply SSourceControlReview::OnLoadChangelistClicked()
{
	LoadChangelist(ChangelistNumText->GetText().ToString());
	return FReply::Handled();
}

bool SSourceControlReview::IsSourceControlActive() const
{
	return FModuleManager::LoadModuleChecked<ISourceControlModule>("SourceControl").IsEnabled();
}

FText SSourceControlReview::LoadChangelistTooltip() const
{
	if (IsSourceControlActive())
	{
		return LOCTEXT("LoadChangelistTooltip", "Load Changelist");
	}
	return LOCTEXT("LoadChangelistInactive", "Enable Revision Control to load changelist");
}

void SSourceControlReview::OnChangelistNumChanged(const FText& Text)
{
	const FString& Data = Text.ToString();

	// Use the longest substring that consists of only valid characters.
	// For example if someone enters:
	// "john.doe2/CL_123456789/version_13", we'll use "123456789" because it's longer than "2" and "13"
	int32 BestMatchBegin = 0;
	int32 BestMatchLen = 0;
	
	FRegexMatcher RegexMatcher(FRegexPattern(TEXT("\\d+")), Data);
	while(RegexMatcher.FindNext())
	{
		const int32 MatchBegin = RegexMatcher.GetMatchBeginning();
		const int32 MatchLen = RegexMatcher.GetMatchEnding() - MatchBegin;
		
		if (MatchLen > BestMatchLen)
		{
			BestMatchBegin = MatchBegin;
			BestMatchLen = MatchLen;
		}
	}
	
	const FText ValidText = FText::FromString(Data.Mid(BestMatchBegin, BestMatchLen));
	ChangelistNumText->SetText(ValidText);

	const TSharedPtr<FChangelistLightInfo> ChangelistLightInfo = MakeShared<FChangelistLightInfo>(ValidText);
	if (!bUncommittedChangelistNum || CLHistory.IsEmpty())
	{
		bUncommittedChangelistNum = true;
		if (CLHistory.Num() > 5)
		{
			CLHistory.Pop();
		}
		CLHistory.Insert(ChangelistLightInfo, 0);
	}
	else
	{
		CLHistory[0] = ChangelistLightInfo;
	}
	ChangelistNumComboBox->RefreshOptions();
	if (ChangelistNumComboBox->GetSelectedItem() != CLHistory[0])
	{
		ChangelistNumComboBox->SetSelectedItem(CLHistory[0]);
	}
}

void SSourceControlReview::OnChangelistNumCommitted(const FText& Text, ETextCommit::Type CommitMethod)
{
	if (CommitMethod == ETextCommit::OnEnter)
	{
		LoadChangelist(Text.ToString());
	}
}

FReply SSourceControlReview::OnEnableCommentsButtonClicked()
{
	if (CommentsAPI)
	{
		NumItemsToLoad += 2;
		CommentsAPI->GetOrCreateReviewTopicForCL(CurrentChangelistInfo.ChangelistNum, FSwarmCommentsAPI::OnGetReviewTopicForCLComplete::CreateRaw(this, &SSourceControlReview::OnGetReviewTopic));
	}
	return FReply::Handled();
}

EVisibility SSourceControlReview::EnableCommentsButtonVisibility() const
{
	if (!CurrentChangelistInfo.ChangelistNum.IsEmpty() && CommentsAPI && !ReviewTopic.IsSet() && !IsLoading())
	{
		return EVisibility::Visible;
	}
	return EVisibility::Collapsed;
}

TSharedRef<SWidget> SSourceControlReview::MakeCLComboOption(TSharedPtr<FChangelistLightInfo> Item) const
{
	FText Text;
	if (!Item->Author.IsEmpty())
	{
		Text = FText::Format(LOCTEXT("CLComboOption", "{0} - {1}"), Item->Number, Item->Author);
	}
	else
	{
		Text =Item->Number;
	}
	
	
	return SNew(STextBlock)
		.Text(Text)
		.ToolTipText(Item->Description);
}

void SSourceControlReview::OnCLComboSelection(TSharedPtr<FChangelistLightInfo> Item, ESelectInfo::Type SelectInfo) const
{
	if (Item.IsValid() && SelectInfo != ESelectInfo::Direct)
	{
		ChangelistNumText->SetText(Item->Number);
	}
}

void SSourceControlReview::SaveCLHistory()
{
	TArray<FString> Numbers;
	TArray<FString> Authors;
	TArray<FString> Descriptions;
	for(const TSharedPtr<FChangelistLightInfo>& Item : CLHistory)
	{
		Numbers.Add(Item->Number.ToString());
		Authors.Add(Item->Author.ToString());
		Descriptions.Add(Item->Description.ToString());
	}
	GConfig->SetArray(TEXT("SourceControlReview"), TEXT("CLHistory.Numbers"), Numbers, GEditorPerProjectIni);
	GConfig->SetArray(TEXT("SourceControlReview"), TEXT("CLHistory.Authors"), Authors, GEditorPerProjectIni);
	GConfig->SetArray(TEXT("SourceControlReview"), TEXT("CLHistory.Descriptions"), Descriptions, GEditorPerProjectIni);
	GConfig->Flush(false, GEditorPerProjectIni);
}

void SSourceControlReview::LoadCLHistory()
{
	TArray<FString> Numbers;
	TArray<FString> Authors;
	TArray<FString> Descriptions;
	GConfig->GetArray(TEXT("SourceControlReview"), TEXT("CLHistory.Numbers"), Numbers, GEditorPerProjectIni);
	GConfig->GetArray(TEXT("SourceControlReview"), TEXT("CLHistory.Authors"), Authors, GEditorPerProjectIni);
	GConfig->GetArray(TEXT("SourceControlReview"), TEXT("CLHistory.Descriptions"), Descriptions, GEditorPerProjectIni);
	CLHistory.Empty();
	if (Numbers.Num() == Authors.Num() && Numbers.Num() == Descriptions.Num())
	{
		for(int32 I = 0; I < Numbers.Num(); ++I)
		{
			CLHistory.Add(MakeShared<FChangelistLightInfo>(
				FText::FromString(Numbers[I]),
				FText::FromString(Authors[I]),
				FText::FromString(Descriptions[I])
			));
		}
	}
	
}

void SSourceControlReview::SetLoading(bool bInLoading)
{
	bChangelistLoading = bInLoading;
	
	// show loading bar and text if we're loading
	LoadingProgressBar->SetVisibility(bInLoading? EVisibility::Visible : EVisibility::Collapsed);
	LoadingTextBlock->SetVisibility(bInLoading? EVisibility::Visible : EVisibility::Collapsed);
	if (bInLoading)
	{
		EnterChangelistTextBlock->SetVisibility(EVisibility::Collapsed);
	}
	else if(ChangelistFiles.IsEmpty())
	{
		EnterChangelistTextBlock->SetVisibility(EVisibility::Visible);
	}
}

bool SSourceControlReview::IsLoading() const
{
	return bChangelistLoading;
}

bool SSourceControlReview::IsChangelistRecordValid(const TArray<TMap<FString, FString>>& InRecord) const
{
	if (InRecord.IsEmpty())
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("ChangelistNotFoundError", "No record found for this changelist"));
		return false;
	}
	if (InRecord.Num() > 1)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("ChangelistInvalidResponseFormat", "Invalid API response from Revision Control"));
		return false;
	}

	const TMap<FString, FString>& RecordMap = InRecord[ReviewHelpers::RecordIndex];
	if (!RecordMap.Contains(ReviewHelpers::ChangelistStatusKey))
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("ChangelistMissingStatus", "Changelist is missing status information"));
		return false;
	}
	if (!RecordMap.Contains(ReviewHelpers::AuthorKey))
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("ChangelistMissingAuthor", "Changelist is missing author information"));
		return false;
	}
	if (!RecordMap.Contains(ReviewHelpers::DescriptionKey))
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("ChangelistMissingDescription", "Changelist is missing description information"));
		return false;
	}
	if (!RecordMap.Contains(ReviewHelpers::TimeKey))
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("ChangelistMissingDate", "Changelist is missing date information"));
		return false;
	}

	return true;
}

void SSourceControlReview::SetFileSourceControlAction(TSharedPtr<FChangelistFileData> ChangelistFileData, const FString& SourceControlAction)
{
	if (!ChangelistFileData.IsValid())
	{
		UE_LOGF(LogSourceControl, Error, "Unable to set revision control action information. No changelist file data");
		return;
	}

	if (SourceControlAction == FString(TEXT("add")))
	{
		ChangelistFileData->FileSourceControlAction = ESourceControlAction::Add;
	}
	else if (SourceControlAction == FString(TEXT("edit")))
	{
		ChangelistFileData->FileSourceControlAction = ESourceControlAction::Edit;
	}
	else if (SourceControlAction == FString(TEXT("delete")))
	{
		ChangelistFileData->FileSourceControlAction = ESourceControlAction::Delete;
	}
	else if (SourceControlAction == FString(TEXT("branch")))
	{
		ChangelistFileData->FileSourceControlAction = ESourceControlAction::Branch;
	}
	else if (SourceControlAction == FString(TEXT("integrate")))
	{
		ChangelistFileData->FileSourceControlAction = ESourceControlAction::Integrate;
	}
	else if (SourceControlAction == FString(TEXT("move/delete")))
	{
		ChangelistFileData->FileSourceControlAction = ESourceControlAction::Delete;
	}
	else if (SourceControlAction == FString(TEXT("move/add")))
	{
		ChangelistFileData->FileSourceControlAction = ESourceControlAction::Add;
	}
	else
	{
		UE_LOGF(LogSourceControl, Error, "Unable to parse revision control action information. '%ls' diff will not be shown", *ChangelistFileData->AssetName);
		ChangelistFileData->FileSourceControlAction = ESourceControlAction::Unset;
	}
}

static FString GetSharedBranchPath(const TMap<FString, FString>& InChangelistRecord)
{
	FString SharedBranchPath;
	if (const FString* Found = InChangelistRecord.Find(ReviewHelpers::FileDepotKey + "0"))
	{
		SharedBranchPath = *Found;
	}
	else
	{
		return FString();
	}

	// TODO: @jordan.hoffmann: Revise for other source controls
	//Each file in p4 has an index "depotFile0" this is the index that is updated by the while loop to find all the files "depotFile1", "depotFile2" etc...
	uint32  RecordFileIndex = 1;
	//The p4 records is the map a file key starts with "depotFile" and is followed by file index 
	FString RecordFileMapKey = ReviewHelpers::FileDepotKey + LexToString(RecordFileIndex);
	int32 TrimIndex = SharedBranchPath.Len();
	while (const FString* Found = InChangelistRecord.Find(RecordFileMapKey))
	{
		// starting from the left, find the portion that's shared between both strings
		int32 CurrentIndex = 0;
		TrimIndex = FMath::Min(Found->Len(), TrimIndex);
		for(; CurrentIndex < TrimIndex; ++CurrentIndex)
		{
			if (SharedBranchPath[CurrentIndex] != (*Found)[CurrentIndex])
			{
				break;
			}
		}
		TrimIndex = CurrentIndex;

		// increment to next file path
		RecordFileMapKey = ReviewHelpers::FileDepotKey + LexToString(++RecordFileIndex);
	}
	// if the shared string is in the middle of a file/directory name, backtrack to the last seen '/'
	TrimIndex = SharedBranchPath.Find(TEXT("/"), ESearchCase::IgnoreCase, ESearchDir::FromEnd, TrimIndex);
	return TrimIndex == INDEX_NONE ? FString() : SharedBranchPath.Left(TrimIndex);
}

void SSourceControlReview::SetChangelistInfo(const TMap<FString, FString>& InChangelistRecord, const FString& ChangelistNum)
{
	CurrentChangelistInfo.Author = FText::FromString(InChangelistRecord[ReviewHelpers::AuthorKey]);
	CurrentChangelistInfo.Description = FText::FromString(InChangelistRecord[ReviewHelpers::DescriptionKey]);
	CurrentChangelistInfo.Status = FText::FromString(InChangelistRecord[ReviewHelpers::ChangelistStatusKey]);
	CurrentChangelistInfo.SharedPath = FText::FromString(GetSharedBranchPath(InChangelistRecord));
	CurrentChangelistInfo.ChangelistNum = ChangelistNum;
}

TSharedRef<ITableRow> SSourceControlReview::OnGenerateFileRow(TSharedPtr<FChangelistFileData> FileData, const TSharedRef<STableViewBase>& Table) const
{
	return SNew(SSourceControlReviewEntry, Table)
	.FileData(*FileData)
	.CommentsAPI(CommentsAPI);
}

FString SSourceControlReview::TrimSharedPath(FString FullCLPath) const
{
	FullCLPath.RemoveFromStart(*CurrentChangelistInfo.SharedPath.ToString());
	return FullCLPath;
}

static bool IsObjectRedirector(FLinkerLoad* LinkerLoad, FStringView AssetName)
{
	// If the asset's name matches the name of a redirector, we know the entire asset is a redirector
	for (const FObjectExport& Export : LinkerLoad->ExportMap)
	{
		if (Export.ObjectName.ToString() == AssetName)
		{
			if (!Export.ClassIndex.IsImport())
			{
				return false;
			}
			const int32 ClassIndex = Export.ClassIndex.ToImport();
			const FObjectImport& AssetClass = LinkerLoad->ImportMap[ClassIndex];
			return AssetClass.ObjectName == NAME_ObjectRedirector;
		}
	}
	return false;
}

// Find the path of a redirector relative to it's content directory
static FString GetContentRelativePath(FLinkerLoad* LinkerLoad)
{
	for (const FObjectImport& Import : LinkerLoad->ImportMap)
    {
        if (Import.ClassName != NAME_Package)
        {
        	continue;
        }
		
        const FString PackageName = Import.ObjectName.ToString();
		
		// ignore internal packages
        if (PackageName.StartsWith(TEXT("/Script/"), ESearchCase::CaseSensitive))
        {
        	continue;
        }

		// Matches after the last content directory in the path
		// if content directory isn't found, assume this is a plugin and match after the first directory
		const FRegexPattern Pattern(TEXT(R"((?:(?:^.*/Content/)|(?:^/[^/]*/))(.*))"));
		FRegexMatcher Regex(Pattern, PackageName);
		if (Regex.FindNext())
		{
			return Regex.GetCaptureGroup(1);
		}
    }
	return FString();
}

// Attempt to find the content directory and remove it from the path
static FString GetContentRelativePath(const TSharedPtr<FChangelistFileData> &File)
{
	// Matches after the last content directory in the path
	const FString BasePath = FPaths::GetBaseFilename(File->AssetDepotPath, /* bRemovePath */ false);
	const FRegexPattern Pattern(TEXT(R"((?:^.*/Content/)(.*))"));
	FRegexMatcher Regex(Pattern, BasePath);
	return Regex.FindNext()? Regex.GetCaptureGroup(1) : FString();
}

// Identify renames or moves withing a CL and merge their redirected path into a single diff entry
void SSourceControlReview::FixupRedirectors()
{
	// Search CL for redirectors and map them by their redirected path
	TMap<FString, TWeakPtr<FChangelistFileData>> RedirectorsFound;
	for (TSharedPtr<FChangelistFileData> ChangelistFile : ChangelistFiles)
	{
		if (ChangelistFile->ReviewFileTempPath.EndsWith(TEXT(".uasset")) || ChangelistFile->ReviewFileTempPath.EndsWith(TEXT(".umap")))
		{
			if (FLinkerLoad* ReviewFileLoad = GetPackageLinker(nullptr, FPackagePath::FromLocalPath(ChangelistFile->ReviewFileTempPath), LOAD_ForDiff | LOAD_DisableCompileOnLoad | LOAD_DisableEngineVersionChecks, nullptr))
			{
				if (IsObjectRedirector(ReviewFileLoad, ChangelistFile->AssetName))
				{
					FString Path = GetContentRelativePath(ReviewFileLoad);
					if (!Path.IsEmpty())
					{
						RedirectorsFound.Add(Path, ChangelistFile.ToWeakPtr());
					}
				}
			}
		}
	}

	// Find files that match the redirected paths and merge the two entries into one
	ChangelistFiles.RemoveAllSwap([&RedirectorsFound](const TSharedPtr<FChangelistFileData> &ChangelistFile)
	{
		if (ChangelistFile->FileSourceControlAction == ESourceControlAction::Add)
		{
			const FString Path = GetContentRelativePath(ChangelistFile);
			if (const TWeakPtr<FChangelistFileData>* Found = RedirectorsFound.Find(Path))
			{
				if (const TSharedPtr<FChangelistFileData>& MergedRenameEntry = Found->Pin())
				{
					MergedRenameEntry->ReviewFileTempPath = ChangelistFile->ReviewFileTempPath;
					MergedRenameEntry->PreviousAssetName = MergedRenameEntry->AssetName;
					MergedRenameEntry->AssetName = ChangelistFile->AssetName;
					MergedRenameEntry->RelativeFilePath = ChangelistFile->RelativeFilePath;
					MergedRenameEntry->AssetFilePath = ChangelistFile->AssetFilePath;
					MergedRenameEntry->PreviousFileRevisionNum = MergedRenameEntry->ReviewFileRevisionNum;
					MergedRenameEntry->ReviewFileRevisionNum = ChangelistFile->ReviewFileRevisionNum;
					return true;
				}
			}
		}
		return false;
	});
}

SHeaderRow::FColumn::FArguments SSourceControlReview::HeaderColumn(FName HeaderName)
{
	FText ColumnLabel;
	TOptional<float> ColumnWidth;
	if (HeaderName == ColumnIds::Status)
	{
		ColumnLabel = LOCTEXT("StatusColumnHeader", "Status");
		ColumnWidth = 100;
	}
	else if (HeaderName == ColumnIds::File)
	{
		ColumnLabel = LOCTEXT("FileColumnHeader", "File");
	}
	else if (HeaderName == ColumnIds::Tools)
	{
		ColumnLabel = LOCTEXT("ToolsColumnHeader", "Tools");
		ColumnWidth = 88;
	}

	return SHeaderRow::Column(HeaderName)
		.FixedWidth(ColumnWidth)
		.HAlignHeader(HeaderName == ColumnIds::File? HAlign_Fill : HAlign_Center)
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Fill)
		.VAlignHeader(VAlign_Fill)
		.HeaderContentPadding(FMargin(10.f, 6.f))
		[
			SNew(STextBlock)
			.Text(ColumnLabel)
			.Font(FStyleFonts::Get().Normal)
		];
}

FString SSourceControlReview::AsAssetPath(const FString& FullCLPath)
{
	const FString ProjectName = FString::Format(TEXT("/{0}/"), {FApp::GetProjectName()});
	const int32 Found = FullCLPath.Find(ProjectName);
	if (Found == INDEX_NONE)
	{
		return FString();
	}
	return UKismetSystemLibrary::GetProjectDirectory() / FullCLPath.RightChop(Found + ProjectName.Len());
}

#undef LOCTEXT_NAMESPACE
