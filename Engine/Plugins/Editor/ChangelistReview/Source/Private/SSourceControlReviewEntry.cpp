// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSourceControlReviewEntry.h"

#include "AssetDefinition.h"
#include "AssetToolsModule.h"
#include "ChangelistReviewModule.h"
#include "ClassIconFinder.h"
#include "DiffUtils.h"
#include "Engine/Blueprint.h"
#include "SourceControlHelpers.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "IAssetTools.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SourceControlReviewEntry"

namespace ReviewEntryConsts
{
	static const FString TempFolder = TEXT("/Temp/");
}

void SSourceControlReviewEntry::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	ChangelistFileData = InArgs._FileData;
	CommentsAPI = InArgs._CommentsAPI;
	SMultiColumnTableRow::Construct(FSuperRowType::FArguments(), InOwnerTableView);

	// figure out how this asset diffs, and bind it to this->DiffMethod
	TryBindDiffMethod();
}

TSharedRef<SWidget> SSourceControlReviewEntry::GenerateWidgetForColumn(const FName& ColumnName)
{
	TSharedPtr<SWidget> InnerContent;
	
	if (ColumnName == SourceControlReview::ColumnIds::Status)
	{
		SAssignNew(InnerContent, SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding(0.f, 0.f, 5.f, 0.f)
		[
			SNew(SImage)
			.DesiredSizeOverride(FVector2D(20.f, 20.f))
			.ColorAndOpacity(this, &SSourceControlReviewEntry::GetSourceControlIconColor)
			.Image(this, &SSourceControlReviewEntry::GetSourceControlIcon)
		]
		+SHorizontalBox::Slot()
		.Padding(5.f, 0.f, 0.f, 0.f)
		[
			SNew(SImage)
			.Visibility(this, &SSourceControlReviewEntry::GetUnreadCommentsIconVisibility)
			.ToolTipText(this, &SSourceControlReviewEntry::GetUnreadCommentsTooltip)
			.DesiredSizeOverride(FVector2D(20.f, 20.f))
			.ColorAndOpacity(FStyleColors::Foreground)
			.Image(FAppStyle::Get().GetBrush(TEXT("Icons.Comment")))
		];
	}
	else if (ColumnName == SourceControlReview::ColumnIds::File)
	{
		SAssignNew(InnerContent, SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(16.f, 1.f, 0.f, 0.f)
		[	
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoWidth()
			.Padding(0.f, 0.f, 8.f, 4.f)
			[
				SNew(SBox)
				.MaxAspectRatio(1.f)
				.MinAspectRatio(1.f)
				.HAlign(HAlign_Left)
				[
							
					SAssignNew(AssetTypeIcon, SImage)
					.DesiredSizeOverride(FVector2D(16.f, 16.f))
					.Image(GetAssetTypeIcon())
					.ToolTipText(GetAssetType())
				]
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(STextBlock)
				.Text(this, &SSourceControlReviewEntry::GetAssetNameText)
				.ColorAndOpacity(FStyleColors::AccentWhite)
				.Font(FStyleFonts::Get().Normal)
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(16.f, 0.f, 0.f, 5.f)
		[
			SNew(SScrollBox)
			.Orientation(EOrientation::Orient_Horizontal)
			+SScrollBox::Slot()
			[
				SNew(STextBlock)
				.Text(this, &SSourceControlReviewEntry::GetLocalAssetPathText)
				.Font(FStyleFonts::Get().Small)
			]
		];
	}
	else if (ColumnName == SourceControlReview::ColumnIds::Tools)
	{
		SAssignNew(InnerContent, SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding(0.f, 0.f, 7.f, 0.f)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.OnClicked(this, &SSourceControlReviewEntry::OnDiffClicked)
	 		.ToolTipText(LOCTEXT("ViewDiffTooltip", "Diff Against Previous Revision"))
			.ContentPadding(FMargin(0.f))
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.DesiredSizeOverride(FVector2D(20.f, 20.f))
				.Image(FAppStyle::Get().GetBrush("SourceControl.Actions.Diff"))
			]
		]
		+SHorizontalBox::Slot()
		.Padding(7.f, 0.f, 0.f, 0.f)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.IsEnabled(this, &SSourceControlReviewEntry::CanBrowseToAsset)
			.OnClicked(this, &SSourceControlReviewEntry::OnBrowseToAssetClicked)
			.ToolTipText(this, &SSourceControlReviewEntry::GetBrowseToAssetTooltip)
			.ContentPadding(FMargin(0.f))
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.DesiredSizeOverride(FVector2D(20.f, 20.f))
				.Image(FAppStyle::Get().GetBrush("Icons.BrowseContent"))
			]
			
		];
	}

	static const FSlateBrush* const WhiteBrush = FAppStyle::GetBrush("WhiteBrush");
	
	return SNew(SBorder)
	.BorderImage(WhiteBrush)
	.BorderBackgroundColor(FStyleColors::AccentBlack)
	.Padding(1.f)
	[
		SNew(SBorder)
		.BorderImage(WhiteBrush)
		.BorderBackgroundColor(FStyleColors::Recessed)
		.Padding(0.f)
		.VAlign(VAlign_Center)
		.HAlign(ColumnName == SourceControlReview::ColumnIds::File? HAlign_Fill : HAlign_Center)
		[
			InnerContent.ToSharedRef()
		]
	];
}

void SSourceControlReviewEntry::SetEntryData(const FChangelistFileData& InChangelistFileData)
{
	ChangelistFileData = InChangelistFileData;
	AssetTypeIcon->SetImage(GetAssetTypeIcon());
	AssetTypeIcon->SetToolTipText(GetAssetType());

	// if asset changed, we might diff differently. rebind the diff method.
	DiffMethod.Unbind();
	TryBindDiffMethod();
}

FReply SSourceControlReviewEntry::OnDiffClicked() const
{
	DiffMethod.Execute();
	return FReply::Handled();
}

FReply SSourceControlReviewEntry::OnBrowseToAssetClicked() const
{
	TArray<FAssetData> Assets;
	USourceControlHelpers::GetAssetData(ChangelistFileData.AssetFilePath, Assets);
	const FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
	AssetToolsModule.Get().SyncBrowserToAssets(Assets);
	return FReply::Handled();
}

FText SSourceControlReviewEntry::GetBrowseToAssetTooltip() const
{
	if (CanBrowseToAsset())
	{
		return LOCTEXT("BrowseToAssetTooltip", "Browse To Asset");
	}
	return  LOCTEXT("CantBrowseToAssetTooltip", "This File is not an Asset");
}

bool SSourceControlReviewEntry::CanBrowseToAsset() const
{
	TArray<FAssetData> Assets;
	return USourceControlHelpers::GetAssetData(ChangelistFileData.AssetFilePath, Assets);
}

void SSourceControlReviewEntry::TryBindDiffMethod()
{
	if (!DiffMethod.IsBound())
	{
		TryBindUAssetDiff();
	}
	if (!DiffMethod.IsBound())
	{
		TryBindTextDiff();
	}
}

bool SSourceControlReviewEntry::CanDiff() const
{
	return DiffMethod.IsBound();
}

static bool IsAssetPath(const FString& Path)
{
	if (Path.IsEmpty())
	{
		return false;
	}
	const EPackageExtension Extension = FPackagePath::ParseExtension(Path);
	return Extension != EPackageExtension::Unspecified && Extension != EPackageExtension::Custom && Extension != EPackageExtension::EmptyString;
}

void SSourceControlReviewEntry::TryBindUAssetDiff()
{
	UObject* ReviewAsset = nullptr;	
	if (IsAssetPath(ChangelistFileData.ReviewFileTempPath))
	{
		const FPackagePath ReviewFileTempPath = FPackagePath::FromLocalPath(ChangelistFileData.ReviewFileTempPath);
		const FPackagePath AssetPath = FPackagePath::FromLocalPath(ChangelistFileData.AssetFilePath);
		if (UPackage* ReviewFilePkg = DiffUtils::LoadPackageForDiff(ReviewFileTempPath, AssetPath))
		{
			ReviewAsset = FindObject<UObject>(ReviewFilePkg, *ChangelistFileData.AssetName);
			if(ReviewAsset && ReviewAsset->IsA<UObjectRedirector>())
			{
				ReviewAsset = Cast<UObjectRedirector>(ReviewAsset)->DestinationObject;
			}
		}
	}
	
	UObject* PreviousAsset = nullptr;
	if (IsAssetPath(ChangelistFileData.PreviousFileTempPath))
	{
		const FPackagePath PreviousFileTempPath = FPackagePath::FromLocalPath(ChangelistFileData.PreviousFileTempPath);
		const FPackagePath AssetPath = FPackagePath::FromLocalPath(ChangelistFileData.AssetFilePath);
		if (UPackage* PreviousFilePkg = DiffUtils::LoadPackageForDiff(PreviousFileTempPath, AssetPath))
		{
			if (ChangelistFileData.PreviousAssetName.IsEmpty())
			{
				PreviousAsset = FindObject<UObject>(PreviousFilePkg, *ChangelistFileData.AssetName);
			}
			else
			{
				PreviousAsset = FindObject<UObject>(PreviousFilePkg, *ChangelistFileData.PreviousAssetName);
			}
		}
	}
	
	if (!ReviewAsset && !PreviousAsset)
	{
		return;
	}

	DiffMethod.BindLambda([this, PreviousAsset, ReviewAsset]
	{
		const UBlueprint* ReviewBlueprint = Cast<UBlueprint>(ReviewAsset);
		const UBlueprint* PreviousBlueprint = Cast<UBlueprint>(PreviousAsset);
		if ((ReviewBlueprint && !ReviewBlueprint->ParentClass) || (PreviousBlueprint && !PreviousBlueprint->ParentClass))
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("ChangelistNotFoundError", "This Blueprint is missing its parent class. Diff results may be incomplete.\n\nDo you need to load this file's plugin/module?\nFor more details, search 'Can't find file.' in the log"));
		}
		
		FAssetToolsModule::GetModule().Get().DiffAssets(
			PreviousAsset,
			ReviewAsset,
			GetPreviousFileRevisionInfo(),
			GetReviewFileRevisionInfo()
		);
	});
}

static FString MakeEmptyTempFile(const FString& AssetName, const FString& Revision)
{
	const FString Path = FString::Printf(TEXT("%sTemp%s-%s.empty"), *FPaths::DiffDir(), *AssetName, *Revision);
	FFileHelper::SaveStringToFile(TEXT(""), *Path);
	return Path;
}

void SSourceControlReviewEntry::TryBindTextDiff()
{
	DiffMethod.BindLambda([this]
	{
		const FString& DiffCommand = GetDefault<UEditorLoadingSavingSettings>()->TextDiffToolPath.FilePath;
		const FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
		
		FString OldFilePath = ChangelistFileData.PreviousFileTempPath;
		FString NewFilePath = ChangelistFileData.ReviewFileTempPath;

		// if this is an add or delete change, diff against an empty file
		if (OldFilePath.IsEmpty())
		{
			const FString& PreviousAssetName = ChangelistFileData.PreviousAssetName.IsEmpty() ? ChangelistFileData.AssetName : ChangelistFileData.PreviousAssetName;
			OldFilePath = MakeEmptyTempFile(PreviousAssetName, ChangelistFileData.PreviousFileRevisionNum);
		}
		if (NewFilePath.IsEmpty())
		{
			NewFilePath = MakeEmptyTempFile(ChangelistFileData.AssetName, ChangelistFileData.PreviousFileRevisionNum);
		}
		
		AssetToolsModule.Get().CreateDiffProcess(DiffCommand, OldFilePath, NewFilePath);
	});
}

FRevisionInfo SSourceControlReviewEntry::GetReviewFileRevisionInfo() const
{
	FRevisionInfo ReviewFileRevisionInfo;
	ReviewFileRevisionInfo.Changelist = ChangelistFileData.ChangelistNum;
	ReviewFileRevisionInfo.Date = ChangelistFileData.ReviewFileDateTime;

	if (ChangelistFileData.FileSourceControlAction == ESourceControlAction::Delete)
	{
		// new revision for removed files don't exist
		ReviewFileRevisionInfo.Revision = TEXT("Does Not Exist");
	}
	else if (ChangelistFileData.ChangelistState == EChangelistState::Pending)
	{
		ReviewFileRevisionInfo.Revision = TEXT("Pending");
	}
	else
	{
		ReviewFileRevisionInfo.Revision = ChangelistFileData.ReviewFileRevisionNum;
	}

	return ReviewFileRevisionInfo;
}

FRevisionInfo SSourceControlReviewEntry::GetPreviousFileRevisionInfo() const
{
	//We need to have valid revision data for some DiffAssets implementations (Although for now we don't have full data on previous file we are showing correct previous revision information)
	FRevisionInfo PreviousFileRevisionInfo;
	PreviousFileRevisionInfo.Changelist = ChangelistFileData.PreviousChangelistNum;
	PreviousFileRevisionInfo.Date = ChangelistFileData.PreviousFileDateTime;
	if (ChangelistFileData.FileSourceControlAction == ESourceControlAction::Add)
	{
		// previous revision for added files don't exist
		PreviousFileRevisionInfo.Revision = TEXT("Does Not Exist");
	}
	else if (ChangelistFileData.PreviousFileRevisionNum.IsEmpty())
	{
		PreviousFileRevisionInfo.Revision = TEXT("0");
	}
	else
	{
		PreviousFileRevisionInfo.Revision = ChangelistFileData.PreviousFileRevisionNum;
	}
	
	return PreviousFileRevisionInfo;
}

const FSlateBrush* SSourceControlReviewEntry::GetSourceControlIcon() const
{
	// setup lookup table for all the brushes so we don't have to re-call FAppStyle::Get().GetBrush every frame
	static TArray<const FSlateBrush*> Brushes = []()
	{
		TArray<const FSlateBrush*> Temp;
		Temp.AddZeroed(static_cast<uint8>(ESourceControlAction::ActionCount));
		
		//Source control images can be found at Engine\Content\Slate\Starship\SourceControl
		Temp[static_cast<uint8>(ESourceControlAction::Add)] = FAppStyle::Get().GetBrush(TEXT("SourceControl.Add"));
		Temp[static_cast<uint8>(ESourceControlAction::Edit)] = FAppStyle::Get().GetBrush(TEXT("SourceControl.Edit"));
		Temp[static_cast<uint8>(ESourceControlAction::Delete)] = FAppStyle::Get().GetBrush(TEXT("SourceControl.Delete"));
		Temp[static_cast<uint8>(ESourceControlAction::Branch)] = FAppStyle::Get().GetBrush(TEXT("SourceControl.Branch"));
		Temp[static_cast<uint8>(ESourceControlAction::Integrate)] = FAppStyle::Get().GetBrush(TEXT("SourceControl.Integrate"));
		Temp[static_cast<uint8>(ESourceControlAction::Unset)] = FAppStyle::Get().GetBrush(TEXT("SourceControl.Edit"));
		
		return MoveTemp(Temp);
	}();

	uint8 Index = static_cast<uint8>(ChangelistFileData.FileSourceControlAction);
	if (!Brushes.IsValidIndex(Index))
	{
		Index = static_cast<uint8>(ESourceControlAction::Unset);
	}
	return Brushes[Index];
}

FSlateColor SSourceControlReviewEntry::GetSourceControlIconColor() const
{
	switch(ChangelistFileData.FileSourceControlAction)
	{
	case ESourceControlAction::Add:
		return FAppStyle::Get().GetSlateColor("SourceControl.Diff.AdditionColor");
	case ESourceControlAction::Edit:
		return FAppStyle::Get().GetSlateColor("SourceControl.Diff.MajorModificationColor");
	case ESourceControlAction::Delete:
		return FAppStyle::Get().GetSlateColor("SourceControl.Diff.SubtractionColor");
	
	case ESourceControlAction::Branch:
	case ESourceControlAction::Integrate:
		return FAppStyle::Get().GetSlateColor("SourceControl.Diff.MinorModificationColor");
	
	default: return FStyleColors::Foreground.GetSpecifiedColor();
	}
}

const FSlateBrush* SSourceControlReviewEntry::GetAssetTypeIcon()
{
	if (const UClass* IconClass = ChangelistFileData.GetIconClass())
	{
		if (const FSlateBrush* FileTypeBrush = FClassIconFinder::FindThumbnailForClass(IconClass))
		{
			return FileTypeBrush;
		}
	}
	return FAppStyle::GetBrush("ContentBrowser.ColumnViewAssetIcon");
}

FText SSourceControlReviewEntry::GetAssetType()
{
	if (const UClass* IconClass = ChangelistFileData.GetIconClass())
	{
		return FText::FromString(IconClass->GetName());
	}
	
	int32 ChopIndex;
	ChangelistFileData.AssetFilePath.FindLastChar('.', ChopIndex);
	return FText::Format(LOCTEXT("NonAssetFileType", "{0} File"), FText::FromString(ChangelistFileData.AssetFilePath.RightChop(ChopIndex)));
}

FText SSourceControlReviewEntry::GetAssetNameText() const
{
	if (!ChangelistFileData.PreviousAssetName.IsEmpty())
	{
		return FText::Format(
			LOCTEXT("RenamedAssetFormat", "{0} - (Renamed from: {1})"),
			FText::FromString(ChangelistFileData.AssetName),
			FText::FromString(ChangelistFileData.PreviousAssetName)
			);
	}
	return FText::FromString(ChangelistFileData.AssetName);
}

FText SSourceControlReviewEntry::GetLocalAssetPathText() const
{
	return FText::FromString(TEXT("...") + ChangelistFileData.RelativeFilePath);
}

const TArray<FReviewComment>* SSourceControlReviewEntry::GetReviewComments() const
{
	const TSharedPtr<SSourceControlReview> Review = FChangelistReviewModule::Get().GetActiveReview().Pin();
	return Review? Review->GetReviewCommentsForFile(ChangelistFileData.AssetDepotPath) : nullptr;
}

FString SSourceControlReviewEntry::GetReviewerUsername() const
{
	const TSharedPtr<IReviewCommentAPI> ReviewComments = CommentsAPI.Pin();
	return ReviewComments? ReviewComments->GetUsername() : FString();
}

static bool IsCommentUnread(const FReviewComment& Comment, const FString& Username)
{
	// treat archived comments as read
	if (Comment.bIsClosed)
	{
		return false;
	}
	// if there is no 'ReadBy' set, then the comment is unread
	if (!Comment.ReadBy.IsSet())
	{
		return true;
	}
	// if the logged in user isn't in the 'ReadBy' set, there's an unread comment
	if (!Comment.ReadBy->Contains(Username))
	{
		return true;
	}
	return false;
}

EVisibility SSourceControlReviewEntry::GetUnreadCommentsIconVisibility() const
{
	const TArray<FReviewComment>* Comments = GetReviewComments();
	const FString Username = GetReviewerUsername();
	if (!Comments || Username.IsEmpty())
	{
		return EVisibility::Collapsed;
	}
	
	for (const FReviewComment& Comment : *Comments)
	{
		if (IsCommentUnread(Comment, Username))
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

FText SSourceControlReviewEntry::GetUnreadCommentsTooltip() const
{
	const TArray<FReviewComment>* Comments = GetReviewComments();
	const FString Username = GetReviewerUsername();
	if (!Comments || Username.IsEmpty())
	{
		return {};
	}
	
	int32 NumUnreadComments = 0;
	for (const FReviewComment& Comment : *Comments)
	{
		if (IsCommentUnread(Comment, Username))
		{
			++NumUnreadComments;
		}
	}
	
	return FText::Format(LOCTEXT("UnreadCommentsTooltip", "{0} Unread {0}|plural(one=Comment,other=Comments)"), NumUnreadComments);
}

const FString& SSourceControlReviewEntry::GetSearchableString() const
{
	return ChangelistFileData.AssetName;
}

UBlueprint* SSourceControlReviewEntry::GetOrCreateBlueprintForDiff(UClass* InGeneratedClass, EBlueprintType InBlueprintType) const
{
	if (!InGeneratedClass || !InGeneratedClass->ClassGeneratedBy)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("ReviewChangelistEntry", "Unable to show the diff for added file because generated class is not valid"));
		return nullptr;
	}
	
	const FString PackageName = ReviewEntryConsts::TempFolder / InGeneratedClass->GetName();
	const FName BPName = InGeneratedClass->GetFName();
	
	UPackage* BlueprintPackage = CreatePackage(*PackageName);
	BlueprintPackage->SetPackageFlags(PKG_ForDiffing);
	
	if (UBlueprint* ExistingBlueprint = FindObject<UBlueprint>(BlueprintPackage, *InGeneratedClass->GetName()))
	{
		return ExistingBlueprint;
	}

	UBlueprint* BlueprintObject = FKismetEditorUtilities::CreateBlueprint(InGeneratedClass, BlueprintPackage, BPName, InBlueprintType,
																		  InGeneratedClass->ClassGeneratedBy->GetClass(),
																		  InGeneratedClass->GetClass(), FName("DiffToolActions"));
	
	if (BlueprintObject)
	{
		FAssetRegistryModule::AssetCreated(BlueprintObject);
	}
	
	return BlueprintObject;
}

#undef LOCTEXT_NAMESPACE
