// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SSourceControlReview.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

enum EBlueprintType : int;
struct FRevisionInfo;

class SImage;

class SSourceControlReviewEntry : public SMultiColumnTableRow<TSharedPtr<SourceControlReview::FChangelistFileData>>
{
public:
	using ESourceControlAction = SourceControlReview::ESourceControlAction;
	using EChangelistState = SourceControlReview::EChangelistState;
	using FChangelistFileData = SourceControlReview::FChangelistFileData;
	
	SLATE_BEGIN_ARGS(SSourceControlReviewEntry) {}
		SLATE_ARGUMENT(FChangelistFileData, FileData);
		SLATE_ARGUMENT(TWeakPtr<IReviewCommentAPI>, CommentsAPI);
	SLATE_END_ARGS()
	
	/** Constructs the widget */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);
	
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
	
	/**
	 * Sets needed data to show diff and browse to asset
	 * @param InChangelistFileData changelist entry information
	 */
	void SetEntryData(const FChangelistFileData& InChangelistFileData);
private:

	/**
	 * Hooks into built in engine diff functionality to show differences between supplied file revisions
	 */
	FReply OnDiffClicked() const;

	/**
	 * Uses engine sync to browse to asset functionality to navigate the content window to file
	 */
	FReply OnBrowseToAssetClicked() const;

	/**
	 * Uses engine sync to browse to asset functionality to navigate the content window to file
	 */
	FText GetBrowseToAssetTooltip() const;

	/**
	 * Checks if we have a valid local file for this entry to sync content window to
	 */
	bool CanBrowseToAsset() const;

	/**
	 * if this file is diffable, bind the diff method to this->DiffMethod
	 */
	void TryBindDiffMethod();

	/**
	 * Checks if this asset is diffable
	 */
	bool CanDiff() const;

	/**
	 * Runs logic to do the diff for UAsset objects
	 */
	void TryBindUAssetDiff();

	/**
	 * Runs logic to do the diff for text based objects
	 */
	void TryBindTextDiff();

	/**
	 * Gets revision information for review file
	 */
	FRevisionInfo GetReviewFileRevisionInfo() const;

	/**
	 * Gets revision information for previous file (Currently Limited to only revision num)
	 */
	FRevisionInfo GetPreviousFileRevisionInfo() const;

	/**
	 * return an icon reflecting this entry's source control action (i.e. add, edit, delete, etc...)
	 */
	const FSlateBrush* GetSourceControlIcon() const;

	/**
	 * return the color associated with this entry's source control action (i.e. add, edit, delete, etc...)
	 */
	FSlateColor GetSourceControlIconColor() const;
	
	/**
	 * return an icon that indicates the type of the asset being displayed
	 */
	const FSlateBrush* GetAssetTypeIcon();
	
	/**
	 * return an icon that indicates the type of the asset being displayed
	 */
	FText GetAssetType();

	/**
	 * Name of the asset being displayed
	 */
	FText GetAssetNameText() const;

	/**
	 * Path of the asset being displayed (relative to the changelist path)
	 */
	FText GetLocalAssetPathText() const;

	const TArray<FReviewComment>* GetReviewComments() const;

	FString GetReviewerUsername() const;

	EVisibility GetUnreadCommentsIconVisibility() const;

	FText GetUnreadCommentsTooltip() const;

	/**
	 * Returns asset name to be used for search/filter 
	 */
	const FString& GetSearchableString() const;
	
	/**
	* Gets or creates a new blueprint of the same class as file about to be diffed to be used for diffing class defaults when there is no previous revision
	*/
	UBlueprint* GetOrCreateBlueprintForDiff(UClass* InGeneratedClass, EBlueprintType InBlueprintType) const;

private:
	// File associated with this entry
	FChangelistFileData ChangelistFileData;
	TWeakPtr<IReviewCommentAPI> CommentsAPI;
	
	// Called to diff this file against it's previous revision
	DECLARE_DELEGATE(FDiffMethod);
	FDiffMethod DiffMethod;

	// Sub-Widgets
	TSharedPtr<SImage> AssetTypeIcon;
};
