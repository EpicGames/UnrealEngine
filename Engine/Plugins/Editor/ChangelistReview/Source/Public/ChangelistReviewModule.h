// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

//#include "SSourceControlReview.h"
#include "Modules/ModuleManager.h"

#define UE_API CHANGELISTREVIEW_API

class SSourceControlReview;
class FSpawnTabArgs;
class SDockTab;
class SWidget;

class UContentBrowserAliasDataSource;

class FChangelistReviewModule : public FDefaultModuleImpl
{
public:
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;
	UE_API void ShowReviewTab();
	UE_API bool CanShowReviewTab() const;
	UE_API TWeakPtr<SSourceControlReview> GetActiveReview();
	
	/**
	 * Opens review tool and loads the provided change list.
	 *
	 * @param[in] Changelist The change list to load.
	 * @return @c true if changelist review tool was opened and change list is put to be loaded, @c false otherwise.
	 */
	UE_API bool OpenChangelistReview(const FString& Changelist);

	static FChangelistReviewModule& Get()
	{
		return FModuleManager::LoadModuleChecked<FChangelistReviewModule>("ChangelistReview");
	}
private:
	UE_API TSharedRef<SDockTab> CreateReviewTab(const FSpawnTabArgs& Args);
	UE_API TSharedPtr<SWidget> CreateReviewUI();
	
	TWeakPtr<SDockTab> ReviewTab;
	TWeakPtr<SSourceControlReview> ReviewWidget;
};

#undef UE_API
