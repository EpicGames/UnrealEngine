// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ISourceControlProvider.h"
#include "Misc/PackagePath.h"
#include "SourceControlOperations.h"
#include "Widgets/Views/SHeaderRow.h"
#include "FSwarmCommentsAPI.h"

template <typename OptionType> class SComboBox;

class SEditableText;
class STableViewBase;
class STextBlock;
template <typename ItemType> class SListView;

class SSourceControlReviewEntry;
class SChangelistEditableTextBox;
class SProgressBar;

namespace SourceControlReview
{
	enum class ESourceControlAction : uint8
	{
		Add,
		Edit,
		Delete,
		Branch,
		Integrate,
		Unset,

		// keep this last
		ActionCount
	};

	enum class EChangelistState : uint8
	{
		Submitted,
		Pending
	};

	struct FChangelistFileData
	{
		FChangelistFileData() = default;

		bool IsDataValidForEntry() const
		{
			if (FileSourceControlAction == ESourceControlAction::Unset)
			{
				return false;
			}
			if (ReviewFileTempPath.IsEmpty() && FileSourceControlAction != ESourceControlAction::Delete)
			{
				return false;
			}
			if (PreviousFileTempPath.IsEmpty() && FileSourceControlAction != ESourceControlAction::Add && FileSourceControlAction != ESourceControlAction::Branch)
			{
				return false;
			}
			
			return true;
		}

		FString AssetName;

		// path where the temporary asset was downloaded to
		FString ReviewFileTempPath;

		FString ReviewFileRevisionNum;

		FDateTime ReviewFileDateTime;

		FString PreviousAssetName;

		// path where the temporary asset was downloaded to
		FString PreviousFileTempPath;

		FString PreviousFileRevisionNum;

		FDateTime PreviousFileDateTime;

		FString RelativeFilePath;

		FString AssetFilePath;
		
		FString AssetDepotPath;

		int32 ChangelistNum = INDEX_NONE;
		
		int32 PreviousChangelistNum = INDEX_NONE;

		EChangelistState ChangelistState = EChangelistState::Submitted;

		ESourceControlAction FileSourceControlAction = ESourceControlAction::Unset;

		const UClass* GetIconClass();
		
	private:
		TOptional<const UClass*> CachedIconClass;
	};
	
	namespace ColumnIds
	{
		inline const FName Status = TEXT("Status");
		inline const FName File = TEXT("File");
		inline const FName Tools = TEXT("Tools");
	}

	struct FChangelistLightInfo
	{
		explicit FChangelistLightInfo(const FText& Number)
			: Number(Number)
		{}
		FChangelistLightInfo(const FText& Number, const FText& Author, const FText& Description)
			: Number(Number), Author(Author), Description(Description)
		{}
		
		FText Number;
		FText Author;
		FText Description;
	};
}


/**
 * Used to select a changelist and diff it's changes
 */
class SSourceControlReview : public SCompoundWidget
{
public:
	using ESourceControlAction = SourceControlReview::ESourceControlAction;
	using EChangelistState = SourceControlReview::EChangelistState;
	using FChangelistFileData = SourceControlReview::FChangelistFileData;
	
	SLATE_BEGIN_ARGS(SSourceControlReview) {}
	SLATE_END_ARGS()
	
	/** Constructs the widget */
	void Construct(const FArguments& InArgs);

	virtual ~SSourceControlReview() override;

	/**
	 * Pulls up changelist record from source control
	 * @param Changelist Changelist number to get
	 */
	void LoadChangelist(const FString& Changelist);

	/**
	 * Open changelist for review together with filling in the UI as if it was loaded for review by user.
     *
	 * @param Changelist CL number to review
	 * @return @c true if changelist was open for review and is loaded, @c false otherwise.
	 */
	bool OpenChangelist(const FString& Changelist);


	/**
	 * Returns an array of review comments that are 
	 * @param FilePath the file to retrieve comments from. (supports AssetDepotPaths and temp local file paths)
	 */
	const TArray<FReviewComment>* GetReviewCommentsForFile(const FString& FilePath);


	void UpdateReviewComments();
	void PostComment(FReviewComment& Comment);
	void EditComment(FReviewComment& Comment);
	FString GetReviewerUsername() const;
	bool IsFileInReview(const FString& File) const;
	
private:

	FString AsDepotPath(const FString& FilePath);
	
	/**
	 * Called when FGetChangelistDetails completes
	 */
	void OnGetChangelistDetails(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult, FString Changelist);
	
	/**
	 * Called when FSwarmCommentsAPI::GetReviewTopicForCL completes
	 */
	void OnGetReviewTopic(const FReviewTopic& Topic, const FString& ErrorMessage);
	
	/**
	 * Called when FSwarmCommentsAPI::OnGetReviewComments completes
	 */
	void OnGetReviewComments(const TArray<FReviewComment>& Comments, const FString& ErrorMessage);
	void OnInitReviewComments(const TArray<FReviewComment>& Comments, const FString& ErrorMessage);
	
	/**
	 * Called when file is retrieved from source control
	 * @param ChangelistFileData carried through worker to be filled with information about retrieved file from source control
	 */
	void OnGetFileFromSourceControl(TSharedPtr<FChangelistFileData> ChangelistFileData);
	
	/**
	 * Called when changelist details, changelist files, and changelist review info has all been retrieved
	 */
	void OnLoadComplete();
	
	FReply OnLoadChangelistClicked();
	bool IsSourceControlActive() const;
	FText LoadChangelistTooltip() const;
	void OnChangelistNumChanged(const FText& Text);
	void OnChangelistNumCommitted(const FText& Text, ETextCommit::Type CommitMethod);
	
	FReply OnEnableCommentsButtonClicked();
	EVisibility EnableCommentsButtonVisibility() const;

	TSharedRef<SWidget> MakeCLComboOption(TSharedPtr<SourceControlReview::FChangelistLightInfo>Item) const;
	void OnCLComboSelection(TSharedPtr<SourceControlReview::FChangelistLightInfo> Item, ESelectInfo::Type SelectInfo) const;
	void SaveCLHistory();
	void LoadCLHistory();
	
	/**
	 * Sets bChangelistLoading and then fires the BP_SetLoading event
	 * @param bInLoading when true flips visibility on loading bar and is also blocking from loading multiple changelists at the same time until loading is finished
	 */
	void SetLoading(bool bInLoading);
	
	/**
	 * True if the tool is currently loading a changelist
	 */
	bool IsLoading() const;

	/**
	 * Checks changelist information to make sure it's valid for the diff tool
	 * @param InRecord changelist record
	 */
	bool IsChangelistRecordValid(const TArray<TMap<FString, FString>>& InRecord) const;

	/**
	 * Sets the source control action type for the loaded file
	 * @param ChangelistFileData carried through worker to be filled with information about retrieved file from source control
	 * @param SourceControlAction source action operation type retrieved from source control
	 */
	static void SetFileSourceControlAction(TSharedPtr<FChangelistFileData> ChangelistFileData,
	                                       const FString& SourceControlAction);

	/**
	 * Sets changelist information Author, Description, Status and Depot name that changelist is going to
	 */
	void SetChangelistInfo(const TMap<FString, FString>& InChangelistRecord, const FString& ChangelistNum);

	TSharedRef<ITableRow> OnGenerateFileRow(TSharedPtr<FChangelistFileData> FileData, const TSharedRef<STableViewBase>& Table) const;

	/**
	 * Removes CurrentChangelistInfo.SharedPath from the beginning of FullCLPath
	 */
	FString TrimSharedPath(FString FullCLPath) const;
	
	void FixupRedirectors();

	static SHeaderRow::FColumn::FArguments HeaderColumn(FName HeaderName);

	/**
	 * Removes the game directory from the beginning of the FullCLPath
	 */
	static FString AsAssetPath(const FString& FullCLPath);

	// helper function that sets bUncommittedChangelistNum to false and upgrades CLHistory[0] from a entry to an official one
	void CommitTempChangelistNumToHistory();
	
	// helper function that sets bUncommittedChangelistNum to false and removes CLHistory[0] if it's uncommitted
	void RemoveUncommittedChangelistNumFromHistory();

	// used for asynchronous changelist loading
	bool bChangelistLoading = false;

	void IncrementItemsLoaded();
	uint32 NumItemsToLoad = 0; // counts the number of API calls (swarm, p4, etc) that need to be loaded.
	uint32 NumItemsLoaded = 0; // counts the number of API calls (swarm, p4, etc) that have been loaded
	TArray<TSharedPtr<FChangelistFileData>> ChangelistFiles;
	TSharedPtr<FGetChangelistDetails> GetChangelistDetailsCommand;
	TArray<TSharedPtr<SourceControlReview::FChangelistLightInfo>> CLHistory;
	bool bUncommittedChangelistNum = false;
	
	TSharedPtr<SComboBox<TSharedPtr<SourceControlReview::FChangelistLightInfo>>> ChangelistNumComboBox;
	TSharedPtr<SEditableText> ChangelistNumText;
	TSharedPtr<STextBlock> EnterChangelistTextBlock;
	TSharedPtr<STextBlock> LoadingTextBlock;
	TSharedPtr<SProgressBar> LoadingProgressBar;
	TSharedPtr<SWidget> ChangelistInfoWidget;
	TSharedPtr<SListView<TSharedPtr<FChangelistFileData>>> ChangelistEntriesWidget;

	// Info about the current chagnelist
	struct FChangelistInfo
	{
		FText Author;
		FText SharedPath;
		FText Status;
		FText Description;
		FString ChangelistNum;
	} CurrentChangelistInfo;

	TSharedPtr<IReviewCommentAPI> CommentsAPI;
	TOptional<FReviewTopic> ReviewTopic;
	// review comments that aren't attached to a specific file
	TArray<FReviewComment> GlobalReviewComments;
	// review comments mapped by the file they're attached to (keyed by both repository path and temp local path)
	TMap<FString, TArray<FReviewComment>> FileReviewComments;
	bool bReviewCommentsDirty = false;

	TMap<FString, FString> TempLocalPathToDepotPath;
};
